[CommonSettings]
SourcePath=Plugins/Online/Android/OnlineSubsystemGooglePlay/Content/Localization/OnlineSubsystemGooglePlay
DestinationPath=Plugins/Online/Android/OnlineSubsystemGooglePlay/Content/Localization/OnlineSubsystemGooglePlay
ManifestName=OnlineSubsystemGooglePlay.manifest
ArchiveName=OnlineSubsystemGooglePlay.archive
PortableObjectName=OnlineSubsystemGooglePlay.po
ResourceName=OnlineSubsystemGooglePlay.locres
NativeCulture=en
CulturesToGenerate=en
CulturesToGenerate=fr
CulturesToGenerate=de
CulturesToGenerate=es
CulturesToGenerate=es-419
CulturesToGenerate=it
CulturesToGenerate=ja
CulturesToGenerate=ko
CulturesToGenerate=ru
CulturesToGenerate=zh-Hans
CulturesToGenerate=ar
CulturesToGenerate=pl
CulturesToGenerate=pt-BR

;Gather text from source code
[GatherTextStep0]
CommandletClass=GatherTextFromSource
SearchDirectoryPaths=Plugins/Online/Android/OnlineSubsystemGooglePlay/Source/
FileNameFilters=*.cpp
FileNameFilters=*.h
FileNameFilters=*.c
FileNameFilters=*.inl
FileNameFilters=*.mm
FileNameFilters=*.ini
ShouldGatherFromEditorOnlyData=false

;Write Manifest
[GatherTextStep1]
CommandletClass=GenerateGatherManifest

;Write Archives
[GatherTextStep2]
CommandletClass=GenerateGatherArchive

;Import localized PO files
[GatherTextStep3]
CommandletClass=InternationalizationExport
bImportLoc=true

;Write Localized Text Resource
[GatherTextStep4]
CommandletClass=GenerateTextLocalizationResource

;Export  PO files
[GatherTextStep5]
CommandletClass=InternationalizationExport
bExportLoc=true