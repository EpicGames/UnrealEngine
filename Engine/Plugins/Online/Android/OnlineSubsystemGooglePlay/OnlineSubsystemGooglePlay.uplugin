{
	"FileVersion" : 3,
	
	"FriendlyName" : "Online Subsystem GooglePlay",
	"Version" : 1,
	"VersionName" : "1.0",
	"Description" : "Access to GooglePlay platform",
	"Category" : "Online Platform",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"EnabledByDefault" : true,
	"SupportedTargetPlatforms": [
		"Android"
	],
    "Modules": [
        {
            "Name": "OnlineSubsystemGooglePlay",
            "Type": "Runtime",
            "LoadingPhase": "Default",
            "PlatformAllowList" :
			[
				"Android"
			]
        }
    ],
	"LocalizationTargets": [
		{
			"Name": "OnlineSubsystemGooglePlay",
			"LoadingPolicy": "Always"
		}
	],
	"Plugins": [
		{
			"Name": "OnlineSubsystem",
			"Enabled": true
		},
		{
			"Name": "AndroidPermission",
			"Enabled": true
		}
	]
}