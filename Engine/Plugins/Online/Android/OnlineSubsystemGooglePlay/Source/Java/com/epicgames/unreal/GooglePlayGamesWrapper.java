// Copyright Epic Games, Inc. All Rights Reserved.

package com.epicgames.unreal;

import android.app.Activity;
import android.content.Context;

import androidx.annotation.NonNull;

import com.google.android.gms.games.AchievementsClient;
import com.google.android.gms.games.GamesSignInClient;
import com.google.android.gms.games.LeaderboardsClient;
import com.google.android.gms.games.achievement.Achievement;
import com.google.android.gms.games.achievement.AchievementBuffer;
import com.google.android.gms.games.leaderboard.LeaderboardScore;
import com.google.android.gms.games.leaderboard.LeaderboardVariant;
import com.google.android.gms.games.PlayGames;
import com.google.android.gms.games.PlayGamesSdk;
import com.google.android.gms.games.Player;
import com.google.android.gms.tasks.Task;
import com.google.android.gms.tasks.Tasks;

import java.util.ArrayList;
import java.util.List;

public class GooglePlayGamesWrapper
{
	public static Logger Log = new Logger("UE", "GooglePlayGamesWrapper");

	private static class AuthCodeSettings
	{
		public AuthCodeSettings(String InServerClientId, boolean InForceRefreshToken)
		{
			ServerClientId = InServerClientId;
			ForceRefreshToken = InForceRefreshToken;
		}

		public String ServerClientId;
		public boolean ForceRefreshToken;
	}

	// This method should be called from the onCreate method on the Application
	static public void Initialize(Context AppContext)
	{
		Log.debug("Initializing");
		PlayGamesSdk.initialize(AppContext);
	}

	// Login method to be accessed through JNI
	public static void Login(long TaskPtr, final Activity Activity, String InServerClientId, boolean InForceRefreshToken)
	{
		Log.debug("Login started");
		GamesSignInClient SignInClient = PlayGames.getGamesSignInClient(Activity);

		SignInClient.isAuthenticated().addOnCompleteListener(isAuthenticatedTask ->
			{
				if (isAuthenticatedTask.isSuccessful())
				{
					final AuthCodeSettings AuthCodeData = (InServerClientId != null) ? new AuthCodeSettings(InServerClientId, InForceRefreshToken) : null;
					if (isAuthenticatedTask.getResult().isAuthenticated())
					{
						Log.debug("User already authenticated. Requesting player data");
						RequestIdentityData(TaskPtr, Activity, AuthCodeData);
					}
					else
					{
						Log.debug("User not yet authenticated. Attempting sign in");
						SignInClient.signIn().addOnCompleteListener(SignInTask ->
							{
								if (SignInTask.isSuccessful())
								{
									if (SignInTask.getResult().isAuthenticated())
									{
										Log.debug("Sign in succeeded. Requesting player data");
										RequestIdentityData(TaskPtr, Activity, AuthCodeData);
									}
									else
									{
										LoginFailed(TaskPtr, "Sign in failed");
									}
								}
								else
								{
									LoginFailed(TaskPtr, "GamesSignInClient.signIn task failed. Maybe service was interrupted or wrong configuration?");
								}
							});
					}
				}
				else
				{
					LoginFailed(TaskPtr, "GamesSignInClient.isAuthenticated task failed. Maybe service was interrupted or wrong configuration?");
				}
			});
	}

	private static Task<String> TryRequestAuthCode(Activity Activity, AuthCodeSettings Settings)
	{
		if (Settings == null)
		{
			return null;
		}
		Log.debug("Will request auth code");

		GamesSignInClient SignInClient = PlayGames.getGamesSignInClient(Activity);
		return SignInClient.requestServerSideAccess(Settings.ServerClientId, Settings.ForceRefreshToken);
	}

	private static void RequestIdentityData(long TaskPtr, Activity Activity, AuthCodeSettings Settings)
	{
		List<Task<?>> LoginTasks = new ArrayList<>();

		final Task<Player> GetPlayerTask = PlayGames.getPlayersClient(Activity).getCurrentPlayer();
		LoginTasks.add(GetPlayerTask);

		final Task<String> GetAuthCodeTask = TryRequestAuthCode(Activity, Settings);
		if (GetAuthCodeTask != null)
		{
			LoginTasks.add(GetAuthCodeTask);
		}

		Tasks.whenAllComplete(LoginTasks).addOnCompleteListener(CompositeTask ->
			{
				// Task returned by whenAllComplete always succeed
				if (GetPlayerTask.isSuccessful())
				{
					Player CurrentPlayer = GetPlayerTask.getResult();
					String AuthCode = null;
					if (GetAuthCodeTask != null)
					{
						if (GetAuthCodeTask.isSuccessful())
						{
							AuthCode = GetAuthCodeTask.getResult();
						}
						else
						{
							Log.warn("Could not get auth code. Maybe service was interrupted or wrong configuration?");
						}
					}
					LoginSuccess(TaskPtr, CurrentPlayer, AuthCode);
				}
				else
				{
					LoginFailed(TaskPtr, "PlayersClient.getCurrentPlayer task failed. Maybe service was interrupted or wrong configuration?");
				}
			});
	}

	static void LoginSuccess(long TaskPtr, @NonNull Player Player, String AuthCode)
	{
		Log.debug("Login succeeded");
		nativeLoginSuccess(TaskPtr, Player.getPlayerId(), Player.getDisplayName(), AuthCode);
	}

	static void LoginFailed(long TaskPtr, @NonNull String ErrorMessage)
	{
		Log.warn(ErrorMessage);
		nativeLoginFailed(TaskPtr);
	}

	public static void RequestPlayerLeaderboardScore(long TaskPtr, Activity Activity, String LeaderboardId)
	{
		PlayGames.getLeaderboardsClient(Activity).loadCurrentPlayerLeaderboardScore(LeaderboardId, LeaderboardVariant.TIME_SPAN_ALL_TIME,  LeaderboardVariant.COLLECTION_PUBLIC)
			.addOnCompleteListener(LoadScoreTask ->
			{
				if (LoadScoreTask.isSuccessful())
				{
					Log.debug("Score received for leaderboard" + LeaderboardId);
					LeaderboardScore Result = LoadScoreTask.getResult().get();

					if (Result == null)
					{
						Log.warn("Player not yet in the leaderboard " + LeaderboardId);
					}
					else
					{
						nativeLeaderboardRequestSuccess(TaskPtr,
								Result.getScoreHolder().getDisplayName(),
								Result.getScoreHolder().getPlayerId(),
								Result.getRank(),
								Result.getRawScore());
						return;
					}
				}
				nativeLeaderboardRequestFailed(TaskPtr);
			}); 
	}

	public static void SubmitLeaderboardsScores(long TaskPtr, Activity Activity, String[] Leaderboards, long[] NewScores)
	{
		if (Leaderboards.length != NewScores.length)
		{
			Log.warn("Missmatch in leaderboards/scores array sizes");
		}

		LeaderboardsClient Client = PlayGames.getLeaderboardsClient(Activity); 
		int Count = Math.min(Leaderboards.length, NewScores.length);
		List<Task<?>> UpdateLeaderboardTasks = new ArrayList<>();

		for( int Idx = 0; Idx < Count; Idx ++)
		{
			UpdateLeaderboardTasks.add(Client.submitScoreImmediate(Leaderboards[Idx], NewScores[Idx]));
		}

		Tasks.whenAllComplete(UpdateLeaderboardTasks).addOnCompleteListener(CompositeTask ->
			{
				Log.debug("Submit leaderboards scores response received");

				List<Task<?>> Tasks = CompositeTask.getResult();
				boolean bSuccess = true;
				for (int Idx = 0; Idx < Count; Idx ++)
				{
					Task<?> Task = Tasks.get(Idx);
					if (!Task.isSuccessful())
					{
						bSuccess = false;
						Log.warn("Failed to update leaderboard " + Leaderboards[Idx] + ":" + Task.getException().getMessage());
					}
				}

				nativeFlushLeaderboardsCompleted(TaskPtr, bSuccess);
			});
	}

	public static void ShowLeaderboardUI(Activity Activity, String LeaderboardId)
	{
		PlayGames.getLeaderboardsClient(Activity)
			.getLeaderboardIntent(LeaderboardId)
			.addOnSuccessListener(Intent -> Activity.startActivityForResult(Intent, GameActivity.REQUEST_CODE_SHOW_GOOGLEPLAY_UI));
	}

	public static void QueryAchievements(long TaskPtr, Activity Activity)
	{
		PlayGames.getAchievementsClient(Activity)
			.load(false)
			.addOnCompleteListener(LoadAchievementsTask ->
			{
				if (LoadAchievementsTask.isSuccessful())
				{
					Log.debug("Achievements data received");

					AchievementBuffer Achievements = LoadAchievementsTask.getResult().get();
					int Count = Achievements.getCount();

					String[] AchievementIds = new String[Count];
					int[] Types = new int[Count];
					int[] Steps = new int[Count];
					int[] TotalSteps = new int[Count];
					String[] Titles = new String[Count];
					String[] Descriptions = new String[Count];
					boolean[] IsHidden = new boolean[Count];
					long[] LastUpdatedTimestamps = new long[Count];

					for (int Idx = 0; Idx < Count; Idx++)
					{
						Achievement Current = Achievements.get(Idx);

						AchievementIds[Idx] = Current.getAchievementId(); 
						Types[Idx] = Current.getType();
						Titles[Idx] = Current.getName();
						Descriptions[Idx] = Current.getDescription();
						IsHidden[Idx] = Current.getState() == Achievement.STATE_HIDDEN;
						LastUpdatedTimestamps[Idx] = Current.getLastUpdatedTimestamp();

						if (Current.getType() == Achievement.TYPE_INCREMENTAL)
						{
							Steps[Idx] = Current.getCurrentSteps();
							TotalSteps[Idx] = Current.getTotalSteps();
						}
						else //Achievement.TYPE_STANDARD
						{
							TotalSteps[Idx] = 0;
							if (Current.getState() == Achievement.STATE_UNLOCKED)
							{
								Steps[Idx] = 1;
							}
							else
							{
								Steps[Idx] = 0;
							}
						}
					}

					nativeQueryAchievementsSuccess(TaskPtr, AchievementIds, Types, Steps, TotalSteps, Titles, Descriptions, IsHidden, LastUpdatedTimestamps);
				}
				else
				{
					Log.warn("Failed to get achievements data: " + LoadAchievementsTask.getException().getMessage());
					nativeQueryAchievementsFailed(TaskPtr);
				}
			});
	}

	static final int WRITE_ACHIEVEMENT_ACTION_UNLOCK = 0;
	static final int WRITE_ACHIEVEMENT_ACTION_WRITE_STEPS = 1;
	
	public static void WriteAchievements(long TaskPtr, Activity Activity, String[] AchievementIds, int[] Actions, int[] Values)
	{
		if (AchievementIds.length != Actions.length || AchievementIds.length != Values.length)
		{
			Log.warn("Missmatch in leaderboards/scores array sizes");
		}

		AchievementsClient Client = PlayGames.getAchievementsClient(Activity);
		List<Task<?>> WriteTasks = new ArrayList<>();

		int Count = Math.min(Math.min(AchievementIds.length, Actions.length), Values.length);
		for (int Idx = 0; Idx < Count; Idx ++)
		{
			switch ( Actions[Idx] )
			{
				case WRITE_ACHIEVEMENT_ACTION_UNLOCK:
				{
					WriteTasks.add(Client.unlockImmediate(AchievementIds[Idx]));
				}
				break;
				case WRITE_ACHIEVEMENT_ACTION_WRITE_STEPS:
				{
					WriteTasks.add(Client.setStepsImmediate(AchievementIds[Idx], Values[Idx]));
				}
				break;
			}
		}
		Tasks.whenAllComplete(WriteTasks).addOnCompleteListener(CompositeTask ->
			{
				Log.debug("Write achievements response received");

				List<Task<?>> Tasks = CompositeTask.getResult();
				List<String> Succeeded = new ArrayList<>();
				for (int Idx = 0; Idx < Count; Idx ++)
				{
					Task<?> Task = Tasks.get(Idx);
					if (Task.isSuccessful())
					{
						Succeeded.add(AchievementIds[Idx]);
					}
					else
					{
						Log.warn("Failed to write achievement" + AchievementIds[Idx] + ":" + Task.getException().getMessage());
					}
				}
				nativeWriteAchievementsCompleted(TaskPtr, Succeeded.toArray(new String[0]));
			});
	}

	public static void ShowAchievementsUI(Activity Activity)
	{
		PlayGames.getAchievementsClient(Activity)
			.getAchievementsIntent()
			.addOnSuccessListener(Intent -> Activity.startActivityForResult(Intent, GameActivity.REQUEST_CODE_SHOW_GOOGLEPLAY_UI));
	}

	native static void nativeLoginSuccess(long TaskPtr, String PlayerId, String DisplayName, String AuthCode);
	native static void nativeLoginFailed(long TaskPtr);

	native static void nativeLeaderboardRequestSuccess(long TaskPtr, String DisplayName, String PlayerId, long Rank, long RawScore);
	native static void nativeLeaderboardRequestFailed(long TaskPtr);
	native static void nativeFlushLeaderboardsCompleted(long TaskPtr, boolean Success);

	native static void nativeQueryAchievementsSuccess(long TaskPtr, String[] AchievementIds, int[] Types, int[] Steps, int[] TotalSteps, String[] Titles, String[] Descriptions, boolean[] IsHidden, long[] LastUpdatedTimestamps);
	native static void nativeQueryAchievementsFailed(long TaskPtr);

	native static void nativeWriteAchievementsCompleted(long TaskPtrr, String[] Succeeded);
}
