// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using EpicGames.Core;
using UnrealBuildTool;

public class OnlineSubsystemGooglePlay : ModuleRules
{
	public OnlineSubsystemGooglePlay(ReadOnlyTargetRules Target) : base(Target)
    {
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		ConfigCache.ReadSettings(DirectoryReference.FromFile(Target.ProjectFile), Target.Platform, this);

		PublicDefinitions.Add("ONLINESUBSYSTEMGOOGLEPLAY_PACKAGE=1");
        
		PrivateIncludePaths.AddRange(
			new string[] {
				"Private",    
			}
			);

		PrivateDependencyModuleNames.AddRange(
			new string[] { 
				"Core",
				"Engine",
				"CoreOnline",
				"OnlineSubsystem", 
				"AndroidRuntimeSettings",
				"Launch"
            }
			);

        string PluginPath = Utils.MakePathRelativeTo(ModuleDirectory, Target.RelativeEnginePath);
        AdditionalPropertiesForReceipt.Add("AndroidPlugin", Path.Combine(PluginPath, "OnlineSubsystemGooglePlay_UPL.xml"));
    }
}
