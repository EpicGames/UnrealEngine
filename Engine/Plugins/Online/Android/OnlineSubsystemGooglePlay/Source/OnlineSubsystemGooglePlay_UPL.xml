<?xml version="1.0" encoding="utf-8"?>
<!--OnlineSubsystemGooglePlay plugin additions-->
<root xmlns:android="http://schemas.android.com/apk/res/android">	
	<init>
		<log text="Subsystem Google Play SDK Android init"/>
		<setBoolFromProperty result="bEnableGooglePlaySupport" ini="Engine" section="/Script/AndroidRuntimeSettings.AndroidRuntimeSettings" property="bEnableGooglePlaySupport" default="false"/>
		<setBoolFromProperty result="bSupportsInAppPurchasing" ini="Engine" section="OnlineSubsystemGooglePlay.Store" property="bSupportsInAppPurchasing" default="false"/>
		<setIntFromProperty result="RequestCodeForPlayGamesActivities" ini="Engine" section="/Script/AndroidRuntimeSettings.AndroidRuntimeSettings" property="RequestCodeForPlayGamesActivities" default="80002"/>
		<dumpvars/>
	</init>
	
	<prebuildCopies>
		<if condition="bEnableGooglePlaySupport">
			<true>
				<log text="Copying Google Play wrapper java file"/>
				<copyFile src="$S(PluginDir)/Java/com/epicgames/unreal/GooglePlayGamesWrapper.java"
						dst = "$S(BuildDir)/src/com/epicgames/unreal/GooglePlayGamesWrapper.java" />
				<if condition="bSupportsInAppPurchasing">
					<true>
						<log text="Copying Google Play store helper java file"/>
						<copyFile src="$S(PluginDir)/Java/com/epicgames/unreal/GooglePlayStoreHelper.java"
								dst = "$S(BuildDir)/src/com/epicgames/unreal/GooglePlayStoreHelper.java" />
					</true>
				</if>
			</true>
		</if>
	</prebuildCopies>
	
	<!-- gradle dependencies additions -->
	<buildGradleAdditions>
		<if condition="bEnableGooglePlaySupport">
			<true>
				<if condition="bSupportsInAppPurchasing">
					<true>
						<insert>
			dependencies {
			implementation 'com.android.billingclient:billing:8.1.0'
			}
						</insert>
					</true>
				</if>
			</true>
		</if>
	</buildGradleAdditions>

	<!-- optional additions to proguard -->
	<proguardAdditions>
		<if condition="bEnableGooglePlaySupport">
			<true>
				<insert>
 -keep class com.epicgames.unreal.GooglePlayGamesWrapper {
	public *;
}                   
				</insert>
				<if condition="bSupportsInAppPurchasing">
					<true>
						<insert>
 -keep class com.epicgames.unreal.GooglePlayStoreHelper {
	public *;
}
						</insert>
					</true>
				</if>
			</true>
		</if>
	</proguardAdditions>

	<gameApplicationOnCreateAdditions>
		<if condition="bEnableGooglePlaySupport">
			<true>
				<insert>
		GooglePlayGamesWrapper.Initialize(getApplicationContext());
				</insert>
			</true>
		</if>
	</gameApplicationOnCreateAdditions>

	<gameActivityAndroidThunkJavaIapBeginPurchase>
		<if condition="bEnableGooglePlaySupport">
			<true>
				<if condition="bSupportsInAppPurchasing">
					<true>
						<insert>
public boolean AndroidThunkJava_IapBeginPurchase(String[] ProductIds, String AccountId)
{
	Log.debug("[JAVA] - AndroidThunkJava_IapBeginPurchase");
	boolean bTriggeredPurchase = false;
	if( IapStoreHelper != null )
	{
		// sha-256 the accountId and get the hex string representation
		String ObfuscatedAccountId = null;
		if (AccountId != null)
		{
			try
			{
				MessageDigest md = MessageDigest.getInstance("SHA-256");
				byte[] sha256hash = md.digest(AccountId.getBytes("UTF-8"));
				StringBuilder builder = new StringBuilder(sha256hash.length * 2);
				for (byte b : sha256hash)
				{
					builder.append(String.format("%02x", b));
				}
				ObfuscatedAccountId = builder.toString();
			}
			catch (NoSuchAlgorithmException ae)
			{
			}
			catch (UnsupportedEncodingException ee)
			{
			}
		}
		bTriggeredPurchase = IapStoreHelper.BeginPurchase(ProductIds, ObfuscatedAccountId);
	}
	else
	{
		Log.debug("[JAVA] - Store Helper is invalid");
	}
	return bTriggeredPurchase;
}
						</insert>
					</true>
				</if>
			</true>
		</if>
	</gameActivityAndroidThunkJavaIapBeginPurchase>
	
	<!-- optional additions to GameActivity imports in GameActivity.java -->
	<gameActivityImportAdditions>
		<if condition="bEnableGooglePlaySupport">
			<true>
				<if condition="bSupportsInAppPurchasing">
					<true>
						<insert>
import java.io.UnsupportedEncodingException;
import java.security.MessageDigest;
import java.security.NoSuchAlgorithmException;
						</insert>
					</true>
				</if>
			</true>
		</if>
	</gameActivityImportAdditions>

	<gameActivityClassAdditions>
		<if condition="bEnableGooglePlaySupport">
			<true>
				<insertValue value="public static int REQUEST_CODE_SHOW_GOOGLEPLAY_UI = $I(RequestCodeForPlayGamesActivities);"/>
				<insertNewline/>
			</true>
		</if>
	</gameActivityClassAdditions>

	<gameActivityIapSetupServiceAdditions>
		<if condition="bEnableGooglePlaySupport">
			<true>
				<if condition="bSupportsInAppPurchasing">
					<true>
						<insert>
		if (getPackageManager().checkPermission("com.android.vending.BILLING", getPackageName()) == getPackageManager().PERMISSION_GRANTED)
		{
			IapStoreHelper = new GooglePlayStoreHelper(this, Log, InEnableAlternativeBillingOnly);
			if (IapStoreHelper != null)
			{
				Log.debug("[JAVA] - AndroidThunkJava_IapSetupService - Setup started");
			}
			else
			{
				Log.debug("[JAVA] - AndroidThunkJava_IapSetupService - Failed to setup IAP service");
			}
		}
		else
		{
			Log.debug("[JAVA] - AndroidThunkJava_IapSetupService - You do not have the appropriate permission setup.");
			Log.debug("[JAVA] - AndroidThunkJava_IapSetupService - Please ensure com.android.vending.BILLING is added to the manifest.");
		}
						</insert>
					</true>
				</if>
			</true>
		</if>
	</gameActivityIapSetupServiceAdditions>
	
	<gameActivityOnActivityResultAdditions>
		<if condition="bEnableGooglePlaySupport">
			<true>
				<insert>
	if (requestCode == REQUEST_CODE_SHOW_GOOGLEPLAY_UI)
	{
		Log.debug("[JAVA] - GooglePlay external activity UI closed (achievements or leaderboards)");
	}
				</insert>
			</true>
		</if>
	</gameActivityOnActivityResultAdditions>
</root>
