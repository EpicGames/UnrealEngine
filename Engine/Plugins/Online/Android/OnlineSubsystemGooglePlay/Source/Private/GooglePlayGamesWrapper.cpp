// Copyright Epic Games, Inc. All Rights Reserved.

#include "GooglePlayGamesWrapper.h"
#include "Android/AndroidJavaEnv.h"
#include "Android/AndroidJNI.h"
#include "Android/AndroidApplication.h"
#include "Containers/StringConv.h"
#include "OnlineSubsystem.h"
#include "OnlineAchievementsInterfaceGooglePlay.h"
#include "OnlineAsyncTaskGooglePlayFlushLeaderboards.h"
#include "OnlineAsyncTaskGooglePlayLogin.h"
#include "OnlineAsyncTaskGooglePlayReadLeaderboard.h"
#include "OnlineAsyncTaskGooglePlayQueryAchievements.h"
#include "OnlineAsyncTaskGooglePlayWriteAchievements.h"
#include "OnlineJniGooglePlayGamesWrapper.h"

FGooglePlayGamesWrapper::~FGooglePlayGamesWrapper()
{
	Reset();
}

void FGooglePlayGamesWrapper::Init()
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		PlayGamesWrapperClass = UE::Jni::Class<UE::Jni::FGooglePlayGamesWrapper>;
		UE_CLOG_ONLINE(PlayGamesWrapperClass == nullptr, Error, TEXT("Could not find com.epicgames.unreal.GooglePlayGamesWrapper class to interface with Java side"));

		if (PlayGamesWrapperClass)
		{
			LoginMethodId = FJavaWrapper::FindStaticMethod(Env, PlayGamesWrapperClass, "Login", "(JLandroid/app/Activity;Ljava/lang/String;Z)V", false);
			UE_CLOG_ONLINE(LoginMethodId == nullptr, Error, TEXT("Could not find \"Login\" Java method to interface with Java side"));

			RequestLeaderboardScoreId = FJavaWrapper::FindStaticMethod(Env, PlayGamesWrapperClass, "RequestPlayerLeaderboardScore", "(JLandroid/app/Activity;Ljava/lang/String;)V", false);
			UE_CLOG_ONLINE(RequestLeaderboardScoreId == nullptr, Error, TEXT("Could not find \"RequestPlayerLeaderboardScore\" Java method to interface with Java side"));

			SubmitLeaderboardsScoresId = FJavaWrapper::FindStaticMethod(Env, PlayGamesWrapperClass, "SubmitLeaderboardsScores", "(JLandroid/app/Activity;[Ljava/lang/String;[J)V", false);
			UE_CLOG_ONLINE(SubmitLeaderboardsScoresId == nullptr, Error, TEXT("Could not find \"SubmitLeaderboardsScores\" Java method to interface with Java side"));

			QueryAchievementsId = FJavaWrapper::FindStaticMethod(Env, PlayGamesWrapperClass, "QueryAchievements", "(JLandroid/app/Activity;)V", false);
			UE_CLOG_ONLINE(QueryAchievementsId == nullptr, Error, TEXT("Could not find \"QueryAchievements\" Java method to interface with Java side"));

			WriteAchievementsId = FJavaWrapper::FindStaticMethod(Env, PlayGamesWrapperClass, "WriteAchievements", "(JLandroid/app/Activity;[Ljava/lang/String;[I[I)V", false);
			UE_CLOG_ONLINE(WriteAchievementsId == nullptr, Error, TEXT("Could not find \"WriteAchievements\" Java method to interface with Java side"));

			ShowAchievementsUIId = FJavaWrapper::FindStaticMethod(Env, PlayGamesWrapperClass, "ShowAchievementsUI", "(Landroid/app/Activity;)V", false);
			UE_CLOG_ONLINE(ShowAchievementsUIId == nullptr, Error, TEXT("Could not find \"ShowAchievementsUI\" Java method to interface with Java side"));

			ShowLeaderboardUIId = FJavaWrapper::FindStaticMethod(Env, PlayGamesWrapperClass, "ShowLeaderboardUI", "(Landroid/app/Activity;Ljava/lang/String;)V", false);
			UE_CLOG_ONLINE(ShowLeaderboardUIId == nullptr, Error, TEXT("Could not find \"ShowLeaderboardUI\" Java method to interface with Java side"));
		}
	}
	else
	{
		UE_LOG_ONLINE(Error, TEXT("Missing Java environment"));
	}
}

void FGooglePlayGamesWrapper::Reset()
{
	PlayGamesWrapperClass = nullptr;
	LoginMethodId = nullptr;
	RequestLeaderboardScoreId = nullptr;
	SubmitLeaderboardsScoresId = nullptr;
	QueryAchievementsId = nullptr;
	WriteAchievementsId = nullptr;
	ShowAchievementsUIId = nullptr;
	ShowLeaderboardUIId = nullptr;
}

bool FGooglePlayGamesWrapper::Login(FOnlineAsyncTaskGooglePlayLogin* Task, const FString& InAuthCodeClientId, bool InForceRefreshToken)
{
	if (PlayGamesWrapperClass != nullptr && LoginMethodId != nullptr)
	{
		if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
		{
			FScopedJavaObject<jstring> AuthCodeClientId = FJavaHelper::ToJavaString(Env, InAuthCodeClientId);
			jboolean ForceRefreshToken = InForceRefreshToken ? JNI_TRUE : JNI_FALSE;

			FJavaWrapper::CallStaticVoidMethod(Env, PlayGamesWrapperClass, LoginMethodId, reinterpret_cast<jlong>(Task), AndroidJavaEnv::GetGameActivityThis(), *AuthCodeClientId, ForceRefreshToken);
			return true;
		}
	}
	UE_LOG_ONLINE(Error, TEXT("Could not invoke Login Java method"));
	return false;
}

bool FGooglePlayGamesWrapper::RequestLeaderboardScore(FOnlineAsyncTaskGooglePlayReadLeaderboard* Task, const FString& InLeaderboardId)
{
	if (PlayGamesWrapperClass != nullptr && RequestLeaderboardScoreId != nullptr)
	{
		if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
		{
			FScopedJavaObject<jstring> LeaderboardId = FJavaHelper::ToJavaString(Env, InLeaderboardId);

			FJavaWrapper::CallStaticVoidMethod(Env, PlayGamesWrapperClass, RequestLeaderboardScoreId, reinterpret_cast<jlong>(Task), AndroidJavaEnv::GetGameActivityThis(), *LeaderboardId);
			return true;
		}
	}
	UE_LOG_ONLINE(Error, TEXT("Could not invoke RequestLeaderboardScore Java method"));
	return false;
}

bool FGooglePlayGamesWrapper::FlushLeaderboardsScores(FOnlineAsyncTaskGooglePlayFlushLeaderboards* Task, const TArray<FGooglePlayLeaderboardScore>& Scores)
{
	if (PlayGamesWrapperClass != nullptr && SubmitLeaderboardsScoresId != nullptr)
	{
		if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
		{
			TArray<FStringView> LeaderboardIds;
			auto UpdatedScores = NewScopedJavaObject(Env,Env->NewLongArray(Scores.Num()));
			LeaderboardIds.Reserve(Scores.Num());
			
			jlong* UpdatedScoresArray = Env->GetLongArrayElements(*UpdatedScores, NULL);

			for (int32 Idx = 0; Idx < Scores.Num(); ++Idx)
			{
				const FGooglePlayLeaderboardScore& Score = Scores[Idx]; 
				LeaderboardIds.Add(Score.GooglePlayLeaderboardId);
				UpdatedScoresArray[Idx] = (jlong)Score.Score;
			}

			Env->ReleaseLongArrayElements(*UpdatedScores, UpdatedScoresArray, 0);

			FScopedJavaObject<jobjectArray> LeaderboardIdsArray = FJavaHelper::ToJavaStringArray(Env, LeaderboardIds);

			FJavaWrapper::CallStaticVoidMethod(Env, PlayGamesWrapperClass, SubmitLeaderboardsScoresId, reinterpret_cast<jlong>(Task), AndroidJavaEnv::GetGameActivityThis(), *LeaderboardIdsArray, *UpdatedScores);
			return true;
		}
	}
	UE_LOG_ONLINE(Error, TEXT("Could not invoke SubmitLeaderboardsScores Java method"));
	return false;
}

bool FGooglePlayGamesWrapper::ShowLeaderboardUI(const FString& InLeaderboardId)
{
	if (PlayGamesWrapperClass != nullptr && ShowLeaderboardUIId != nullptr)
	{
		if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
		{
			FScopedJavaObject<jstring> LeaderboardId = FJavaHelper::ToJavaString(Env, InLeaderboardId);

			FJavaWrapper::CallStaticVoidMethod(Env, PlayGamesWrapperClass, ShowLeaderboardUIId, AndroidJavaEnv::GetGameActivityThis(), *LeaderboardId);
			return true;
		}
	}
	UE_LOG_ONLINE(Error, TEXT("Could not invoke ShowLeaderboardUI Java method"));
	return false;
}

bool FGooglePlayGamesWrapper::QueryAchievements(FOnlineAsyncTaskGooglePlayQueryAchievements* Task)
{
	if (PlayGamesWrapperClass != nullptr && QueryAchievementsId != nullptr)
	{
		if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
		{
			FJavaWrapper::CallStaticVoidMethod(Env, PlayGamesWrapperClass, QueryAchievementsId, reinterpret_cast<jlong>(Task), AndroidJavaEnv::GetGameActivityThis());
			return true;
		}
	}
	UE_LOG_ONLINE(Error, TEXT("Could not invoke QueryAchievements Java method"));
	return false;
}

bool FGooglePlayGamesWrapper::WriteAchievements(FOnlineAsyncTaskGooglePlayWriteAchievements* Task, const TArray<FGooglePlayAchievementWriteData>& WriteAchievementsData)
{
	if (PlayGamesWrapperClass != nullptr && WriteAchievementsId != nullptr)
	{
		if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
		{
			FScopedJavaObject<jobjectArray> AchievementIds = NewScopedJavaObject(Env, Env->NewObjectArray(WriteAchievementsData.Num(), FJavaWrapper::JavaStringClass, NULL));
			FScopedJavaObject<jintArray> Actions = NewScopedJavaObject(Env, Env->NewIntArray(WriteAchievementsData.Num()));
			FScopedJavaObject<jintArray>  Steps = NewScopedJavaObject(Env, Env->NewIntArray(WriteAchievementsData.Num()));

			jint* ActionsData = Env->GetIntArrayElements(*Actions, NULL);
			jint* StepsData = Env->GetIntArrayElements(*Steps, NULL);

			for (int32 Idx = 0; Idx < WriteAchievementsData.Num(); ++Idx)
			{
				const FGooglePlayAchievementWriteData& Entry = WriteAchievementsData[Idx];

				jstring Id = Env->NewStringUTF((const char*)StringCast<UTF8CHAR>(*Entry.GooglePlayAchievementId).Get()); 
				Env->SetObjectArrayElement(*AchievementIds, Idx, Id);
				Env->DeleteLocalRef(Id);

				ActionsData[Idx] = static_cast<jint>(Entry.Action);
				StepsData[Idx] = static_cast<jint>(Entry.Steps);
			}

			Env->ReleaseIntArrayElements(*Actions, ActionsData, JNI_COMMIT);
			Env->ReleaseIntArrayElements(*Steps, StepsData, JNI_COMMIT);

			FJavaWrapper::CallStaticVoidMethod(Env, PlayGamesWrapperClass, WriteAchievementsId, reinterpret_cast<jlong>(Task), AndroidJavaEnv::GetGameActivityThis(), *AchievementIds, *Actions, *Steps);

			Env->ReleaseIntArrayElements(*Actions, ActionsData, JNI_ABORT);
			Env->ReleaseIntArrayElements(*Steps, StepsData, JNI_ABORT);

			return true;
		}
	}
	UE_LOG_ONLINE(Error, TEXT("Could not invoke WriteAchievements Java method"));
	return false;
}

bool FGooglePlayGamesWrapper::ShowAchievementsUI()
{
	if (PlayGamesWrapperClass != nullptr && ShowAchievementsUIId != nullptr)
	{
		if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
		{
			FJavaWrapper::CallStaticVoidMethod(Env, PlayGamesWrapperClass, ShowAchievementsUIId, AndroidJavaEnv::GetGameActivityThis());
			return true;
		}
	}
	UE_LOG_ONLINE(Error, TEXT("Could not invoke ShowAchievementsUI Java method"));
	return false;
}

void JNICALL UE::Jni::FGooglePlayGamesWrapper::nativeLoginSuccess(JNIEnv* jEnv, jclass /*Clazz*/, jlong TaskPtr, jstring InPlayerId, jstring InPlayerDisplayName, jstring InAuthCode)
{
	FOnlineAsyncTaskGooglePlayLogin* Task = reinterpret_cast<FOnlineAsyncTaskGooglePlayLogin*>(TaskPtr);

	FString PlayerId = FJavaHelper::FStringFromParam(jEnv, InPlayerId);
	FString PlayerDisplayName = FJavaHelper::FStringFromParam(jEnv, InPlayerDisplayName);
	FString AuthCode = FJavaHelper::FStringFromParam(jEnv, InAuthCode);

	checkf(!PlayerId.IsEmpty(), TEXT("PlayerId cannot be empty after a successful login"));

	UE_LOG_ONLINE(Verbose, TEXT("nativeLoginSuccess PlayerId: %s Alias: %s Token: %s"), *PlayerId, *PlayerDisplayName , *AuthCode);

	Task->SetLoginData(MoveTemp(PlayerId), MoveTemp(PlayerDisplayName), MoveTemp(AuthCode));
	Task->bWasSuccessful = true;
	Task->bIsComplete = true;
}

void JNICALL UE::Jni::FGooglePlayGamesWrapper::nativeLoginFailed(JNIEnv* jEnv, jclass /*Clazz*/, jlong TaskPtr)
{
	FOnlineAsyncTaskGooglePlayLogin* Task = reinterpret_cast<FOnlineAsyncTaskGooglePlayLogin*>(TaskPtr);

	UE_LOG_ONLINE(Verbose, TEXT("nativeLoginFailed"));

	Task->bWasSuccessful = false;
	Task->bIsComplete = true;
}

void JNICALL UE::Jni::FGooglePlayGamesWrapper::nativeLeaderboardRequestSuccess(JNIEnv* jEnv, jclass /*Clazz*/, jlong TaskPtr, jstring InDisplayName, jstring InPlayerId, jlong InRank, jlong InRawScore)
{
	FOnlineAsyncTaskGooglePlayReadLeaderboard* Task = reinterpret_cast<FOnlineAsyncTaskGooglePlayReadLeaderboard*>(TaskPtr);

	FString DisplayName = FJavaHelper::FStringFromParam(jEnv, InDisplayName);
	FString PlayerId = FJavaHelper::FStringFromParam(jEnv, InPlayerId);
	int64 Rank = (int64)InRank;
	int64 RawScore = (int64)InRawScore;

	UE_LOG_ONLINE(Verbose, TEXT("nativeLeaderboardRequestSuccess Alias: %s PlayerId: %s Rank: %lld RawScore: %lld"), *DisplayName, *PlayerId, Rank, RawScore);

	Task->AddScore(DisplayName, PlayerId, Rank, RawScore);
	Task->bWasSuccessful = true;
	Task->bIsComplete = true;
}

void JNICALL UE::Jni::FGooglePlayGamesWrapper::nativeLeaderboardRequestFailed(JNIEnv* jEnv, jclass /*Clazz*/, jlong TaskPtr)
{
	FOnlineAsyncTaskGooglePlayReadLeaderboard* Task = reinterpret_cast<FOnlineAsyncTaskGooglePlayReadLeaderboard*>(TaskPtr);

	UE_LOG_ONLINE(Verbose, TEXT("nativeLeaderboardRequestFailed"));

	Task->bWasSuccessful = false;
	Task->bIsComplete = true;
}

void JNICALL UE::Jni::FGooglePlayGamesWrapper::nativeFlushLeaderboardsCompleted(JNIEnv* jEnv, jclass /*Clazz*/, jlong TaskPtr, jboolean InSuccess)
{
	bool Success = (InSuccess != JNI_FALSE);
	FOnlineAsyncTaskGooglePlayFlushLeaderboards* Task = reinterpret_cast<FOnlineAsyncTaskGooglePlayFlushLeaderboards*>(TaskPtr);

	UE_LOG_ONLINE(Verbose, TEXT("nativeFlushLeaderboardsCompleted %s"), *LexToString(Success));

	Task->bWasSuccessful = Success;
	Task->bIsComplete = true;
}


void JNICALL UE::Jni::FGooglePlayGamesWrapper::nativeQueryAchievementsSuccess(JNIEnv* jEnv, jclass /*Clazz*/, jlong TaskPtr, Java::Lang::TArray<jstring>* InAchievementIds, Java::Lang::TArray<jint>* InTypes, Java::Lang::TArray<jint>* InSteps, Java::Lang::TArray<jint>* InTotalSteps, Java::Lang::TArray<jstring>* InTitles, Java::Lang::TArray<jstring>* InDescriptions, Java::Lang::TArray<jboolean>* InIsHidden, Java::Lang::TArray<jlong>* InLastUpdatedTimestamps)
{
	FOnlineAsyncTaskGooglePlayQueryAchievements* Task = reinterpret_cast<FOnlineAsyncTaskGooglePlayQueryAchievements*>(TaskPtr);

	UE_LOG_ONLINE(Verbose, TEXT("nativeQueryAchievementsSuccess"));

	jsize AchievementIdsCount = jEnv->GetArrayLength(InAchievementIds);
	jsize TypesCount = jEnv->GetArrayLength(InTypes);
	jsize StepsCount = jEnv->GetArrayLength(InSteps);
	jsize TotalStepsCount = jEnv->GetArrayLength(InTotalSteps);
	jsize TitlesCount = jEnv->GetArrayLength(InTitles);
	jsize DescriptionsCount = jEnv->GetArrayLength(InDescriptions);
	jsize IsHiddenCount = jEnv->GetArrayLength(InIsHidden);
	jsize LastUpdatedTimestampsCount = jEnv->GetArrayLength(InLastUpdatedTimestamps);

	ensure( (AchievementIdsCount == TypesCount) &&
			(AchievementIdsCount == StepsCount) && 
			(AchievementIdsCount == TotalStepsCount) &&
			(AchievementIdsCount == TitlesCount) && 
			(AchievementIdsCount == DescriptionsCount) && 
			(AchievementIdsCount == IsHiddenCount) &&
			(AchievementIdsCount == LastUpdatedTimestampsCount) );

	jint* Types = jEnv->GetIntArrayElements(InTypes, NULL);
	jint* Steps = jEnv->GetIntArrayElements(InSteps, NULL);
	jint* TotalSteps = jEnv->GetIntArrayElements(InTotalSteps, NULL);
	jboolean* IsHidden = jEnv->GetBooleanArrayElements(InIsHidden, NULL);
	jlong* LastUpdatedTimestamps = jEnv->GetLongArrayElements(InLastUpdatedTimestamps, NULL);

	TArray<FOnlineAchievementGooglePlay> AchievementData;
	AchievementData.Reserve(AchievementIdsCount);

	TArray<FOnlineAchievementDesc> AchievementDesc;
	AchievementDesc.Reserve(AchievementIdsCount);

	for (jsize Idx = 0; Idx < AchievementIdsCount; Idx++)
	{
		FOnlineAchievementGooglePlay Achievement;
		switch(Types[Idx])
		{
			case 0: // value for com.google.android.gms.games.achievement.Achievement.TYPE_STANDARD
				Achievement.Type = EGooglePlayAchievementType::Standard;
				Achievement.Progress = Steps[Idx] == 1? 100.f : 0.f;
				break;
			case 1: // value for com.google.android.gms.games.achievement.Achievement.TYPE_INCREMENTAL
				Achievement.Type = EGooglePlayAchievementType::Incremental;
				Achievement.Progress = (float)(100. * Steps[Idx] / TotalSteps[Idx]);
				break;
			default:
				UE_LOG_ONLINE(Warning, TEXT("Unexpected achievement type"));
				continue;
		}
		Achievement.Id = FJavaHelper::FStringFromLocalRef(jEnv, (jstring)jEnv->GetObjectArrayElement(InAchievementIds, Idx));
		Achievement.TotalSteps = (int32)TotalSteps[Idx];
		AchievementData.Add(MoveTemp(Achievement));

		FOnlineAchievementDesc& Desc = AchievementDesc.AddDefaulted_GetRef();
		Desc.Title = FText::FromString(FJavaHelper::FStringFromLocalRef(jEnv, (jstring)jEnv->GetObjectArrayElement(InTitles, Idx)));
		Desc.LockedDesc = FText::FromString(FJavaHelper::FStringFromLocalRef(jEnv, (jstring)jEnv->GetObjectArrayElement(InDescriptions, Idx)));
		Desc.UnlockedDesc = Desc.LockedDesc;
		Desc.bIsHidden = IsHidden[Idx] != JNI_FALSE;
		Desc.UnlockTime = FDateTime::FromUnixTimestamp(LastUpdatedTimestamps[Idx]); 
	}

	jEnv->ReleaseIntArrayElements(InTypes, Types, JNI_ABORT);
	jEnv->ReleaseIntArrayElements(InSteps, Steps, JNI_ABORT);
	jEnv->ReleaseIntArrayElements(InTotalSteps, TotalSteps, JNI_ABORT);
	jEnv->ReleaseBooleanArrayElements(InIsHidden, IsHidden, JNI_ABORT);
	jEnv->ReleaseLongArrayElements(InLastUpdatedTimestamps, LastUpdatedTimestamps, JNI_ABORT);

	Task->SetAchievementsData(MoveTemp(AchievementData), MoveTemp(AchievementDesc));

	Task->bWasSuccessful = true;
	Task->bIsComplete = true;
}


void JNICALL UE::Jni::FGooglePlayGamesWrapper::nativeQueryAchievementsFailed(JNIEnv* jEnv, jclass /*Clazz*/, jlong TaskPtr)
{
	FOnlineAsyncTaskGooglePlayQueryAchievements* Task = reinterpret_cast<FOnlineAsyncTaskGooglePlayQueryAchievements*>(TaskPtr);

	UE_LOG_ONLINE(Verbose, TEXT("nativeQueryAchievementsFailed"));

	Task->bWasSuccessful = false;
	Task->bIsComplete = true;
}

void JNICALL UE::Jni::FGooglePlayGamesWrapper::nativeWriteAchievementsCompleted(JNIEnv* jEnv, jclass /*Clazz*/, jlong TaskPtr, Java::Lang::TArray<jstring>* InSucceeded)
{
	FOnlineAsyncTaskGooglePlayWriteAchievements* Task = reinterpret_cast<FOnlineAsyncTaskGooglePlayWriteAchievements*>(TaskPtr);

	TArray<FString> Succeeded = FJavaHelper::ObjectArrayToFStringTArray(jEnv, InSucceeded);
	UE_LOG_ONLINE(Verbose, TEXT("nativeWriteAchievementsCompleted Succeeded: %s"), *FString::Join(Succeeded, TEXT(", ")));

	Task->bWasSuccessful = Task->SetSucceeded(Succeeded);
	Task->bIsComplete = true;
}
