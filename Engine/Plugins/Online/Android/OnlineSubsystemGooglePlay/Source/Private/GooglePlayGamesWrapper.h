// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

THIRD_PARTY_INCLUDES_START
#include <jni.h>
THIRD_PARTY_INCLUDES_END

#include "OnlineAchievementGooglePlayCommon.h"
#include "OnlineLeaderboardGooglePlayCommon.h"

class FOnlineAsyncTaskGooglePlayLogin;
class FOnlineAsyncTaskGooglePlayFlushLeaderboards;
class FOnlineAsyncTaskGooglePlayQueryAchievements;
class FOnlineAsyncTaskGooglePlayReadLeaderboard;
class FOnlineAsyncTaskGooglePlayWriteAchievements;

class FGooglePlayGamesWrapper
{
public:
	FGooglePlayGamesWrapper() = default;
	FGooglePlayGamesWrapper(const FGooglePlayGamesWrapper& Other) = delete;
	FGooglePlayGamesWrapper& operator=(const FGooglePlayGamesWrapper& Other) = delete;
	~FGooglePlayGamesWrapper();

	void Init();
	void Reset();

	bool Login(FOnlineAsyncTaskGooglePlayLogin* Task, const FString& InAuthCodeClientId, bool InForceRefreshToken);
	bool RequestLeaderboardScore(FOnlineAsyncTaskGooglePlayReadLeaderboard* Task, const FString& LeaderboardId);
	bool FlushLeaderboardsScores(FOnlineAsyncTaskGooglePlayFlushLeaderboards* Task, const TArray<FGooglePlayLeaderboardScore>& Scores);
	bool QueryAchievements(FOnlineAsyncTaskGooglePlayQueryAchievements* Task);
	bool WriteAchievements(FOnlineAsyncTaskGooglePlayWriteAchievements* Task, const TArray<FGooglePlayAchievementWriteData>& WriteAchievementsData);
	bool ShowAchievementsUI();
	bool ShowLeaderboardUI(const FString& LeaderboardName);

private:
	jclass PlayGamesWrapperClass = nullptr;
	jmethodID LoginMethodId = nullptr;
	jmethodID RequestLeaderboardScoreId = nullptr;
	jmethodID SubmitLeaderboardsScoresId = nullptr;
	jmethodID QueryAchievementsId = nullptr;
	jmethodID WriteAchievementsId = nullptr;
	jmethodID ShowAchievementsUIId = nullptr;
	jmethodID ShowLeaderboardUIId = nullptr;
};
