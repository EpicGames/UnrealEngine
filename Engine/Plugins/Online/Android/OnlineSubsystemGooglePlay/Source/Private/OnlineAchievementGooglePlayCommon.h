// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/OnlineAchievementsInterface.h"

enum class FGooglePlayAchievementWriteAction : int32
{
	Unlock = 0, // Unlock the achievement
	WriteSteps = 1 // Update achievement steps accomplished
};

struct FGooglePlayAchievementWriteData
{
	// Achievement Id as shown in GooglePlay Console
	FString GooglePlayAchievementId;
	// Action to perform
	FGooglePlayAchievementWriteAction Action = FGooglePlayAchievementWriteAction::Unlock;
	// Value to set in cas ethe action requires it
	int32 Steps = 0;
};

enum class EGooglePlayAchievementType
{
	Incremental, // Allows setting a number of steps until completion
	Standard // Just allows unlocking
};

struct FOnlineAchievementGooglePlay: FOnlineAchievement
{
	// Achievement type
	EGooglePlayAchievementType Type = EGooglePlayAchievementType::Standard;
	// Total number of steps needed to complete in case Type is EGooglePlayAchievementType::Incremental
	int32 TotalSteps = 0;
};
