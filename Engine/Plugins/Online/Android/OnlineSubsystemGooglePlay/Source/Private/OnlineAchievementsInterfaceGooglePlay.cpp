// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineAchievementsInterfaceGooglePlay.h"
#include "AndroidRuntimeSettings.h"
#include "OnlineAsyncTaskGooglePlayQueryAchievements.h"
#include "OnlineAsyncTaskGooglePlayWriteAchievements.h"
#include "OnlineIdentityInterfaceGooglePlay.h"
#include "OnlineSubsystemGooglePlay.h"

FOnlineAchievementsGooglePlay::FOnlineAchievementsGooglePlay( FOnlineSubsystemGooglePlay* InSubsystem )
	: Subsystem(InSubsystem)
{
	// Make sure the Google achievement cache is initialized to an invalid status so we know to query it first
}

TOptional<FString> FOnlineAchievementsGooglePlay::GetUnrealAchievementIdFromGoogleAchievementId(const UAndroidRuntimeSettings* Settings, const FString& GoogleId)
{
	for(const auto& Mapping : Settings->AchievementMap)
	{
		if(Mapping.AchievementID == GoogleId)
		{
			return Mapping.Name;
		}
	}
	return NullOpt;
}

TOptional<FString> FOnlineAchievementsGooglePlay::GetGoogleAchievementIdFromUnrealAchievementId(const UAndroidRuntimeSettings* Settings, const FString& UnrealId)
{
	for(const auto& Mapping : Settings->AchievementMap)
	{
		if(Mapping.Name == UnrealId)
		{
			return Mapping.AchievementID;
		}
	}
	return NullOpt;
}

void FOnlineAchievementsGooglePlay::QueryAchievements(const FUniqueNetId& PlayerId, const FOnQueryAchievementsCompleteDelegate& Delegate)
{
	if (!IsLocalPlayer(PlayerId))
	{
		UE_LOG_ONLINE_ACHIEVEMENTS(Warning, TEXT("QueryAchievements failed because was called using non local player or local player is not logged in"));
		Delegate.ExecuteIfBound(PlayerId, false);
		return;
	}

	Subsystem->QueueAsyncTask(new FOnlineAsyncTaskGooglePlayQueryAchievements(Subsystem, PlayerId.AsShared(), Delegate));
}

bool FOnlineAchievementsGooglePlay::IsLocalPlayer(const FUniqueNetId& PlayerId) const
{
	if (!PlayerId.IsValid())
	{
		return false;
	}
	FOnlineIdentityGooglePlayPtr IdentityInt = Subsystem->GetIdentityGooglePlay();
	FUniqueNetIdPtr LocalPlayerNetId = IdentityInt->GetUniquePlayerId(0);
	return LocalPlayerNetId? *LocalPlayerNetId == PlayerId : false;
}

void FOnlineAchievementsGooglePlay::QueryAchievementDescriptions(const FUniqueNetId& PlayerId, const FOnQueryAchievementsCompleteDelegate& Delegate)
{
	// Just query achievements to get descriptions
	QueryAchievements( PlayerId, Delegate );
}

void FOnlineAchievementsGooglePlay::FinishAchievementWrite(
	const FUniqueNetId& PlayerId, 
	const bool bWasSuccessful,
	FOnlineAchievementsWriteRef WriteObject,
	FOnAchievementsWrittenDelegate Delegate)
{
	// If the achievements cache is invalid, we can't do anything here - need to
	// be able to get achievement type and max steps for incremental achievements.
	if (Achievements.IsEmpty())
	{
		Delegate.ExecuteIfBound(PlayerId, false);
		return;
	}

	TArray<FGooglePlayAchievementWriteData> AchievementData;

	auto Settings = GetDefault<UAndroidRuntimeSettings>();
    bool bAllSucceeded = true;
    
	for (auto& [Key, Stat] : WriteObject->Properties)
	{
		// Create an achievement object which should be reported to the server.
		const FOnlineAchievementGooglePlay* Achievement = Achievements.FindByPredicate([KeyTemp = Key](const FOnlineAchievement& Achievement) { return Achievement.Id == KeyTemp;} );
		
		if (!Achievement)
		{
            UE_LOG_ONLINE_ACHIEVEMENTS(Error, TEXT("Unknown achievement id: %s"), *Key);
            bAllSucceeded = false;
			continue;
		}

		TOptional<FString> GoogleAchievementId = GetGoogleAchievementIdFromUnrealAchievementId(Settings, Achievement->Id);
		if (!GoogleAchievementId)
		{
            UE_LOG_ONLINE_ACHIEVEMENTS(Error, TEXT("No Google achievement Id for achievement id: %s"), *Key);
            bAllSucceeded = false;
			continue;
		}

		float PercentComplete = 0.0f;

		// Setup the percentage complete with the value we are writing from the variant type
		switch (Stat.GetType())
		{
		case EOnlineKeyValuePairDataType::Int32:
			{
				int32 Value;
				Stat.GetValue(Value);
				PercentComplete = (float)Value;
				break;
			}
		case EOnlineKeyValuePairDataType::Float:
			{
				float Value;
				Stat.GetValue(Value);
				PercentComplete = Value;
				break;
			}
		default:
			{
                bAllSucceeded = true;
				UE_LOG_ONLINE_ACHIEVEMENTS(Error, TEXT("FOnlineAchievementsGooglePlay Trying to write an achievement with incompatible format. Not a float or int"));
				continue;
			}
		}


		switch(Achievement->Type)
		{
			case EGooglePlayAchievementType::Incremental:
			{
				float StepFraction = (PercentComplete / 100.0f) * Achievement->TotalSteps;
				int RoundedSteps =  FMath::RoundToInt(StepFraction);

				if(RoundedSteps > 0)
				{
					UE_LOG_ONLINE_ACHIEVEMENTS(Log, TEXT("  Incremental: setting progress to %d"), RoundedSteps);
					FGooglePlayAchievementWriteData& Entry = AchievementData.Emplace_GetRef();
					Entry.GooglePlayAchievementId = *GoogleAchievementId;
					Entry.Action = FGooglePlayAchievementWriteAction::WriteSteps;
					Entry.Steps = RoundedSteps;
				}
				else
				{
					UE_LOG_ONLINE_ACHIEVEMENTS(Log, TEXT("  Incremental: not setting progress to %d"), RoundedSteps);
				}
				break;
			}

			case EGooglePlayAchievementType::Standard:
			{
				// Standard achievements only unlock if the progress is at least 100%.
				if (PercentComplete >= 100.0f)
				{
					UE_LOG_ONLINE_ACHIEVEMENTS(Log, TEXT("  Standard: unlocking"));
					FGooglePlayAchievementWriteData& Entry = AchievementData.Emplace_GetRef();
					Entry.GooglePlayAchievementId = *GoogleAchievementId;
					Entry.Action = FGooglePlayAchievementWriteAction::Unlock;
				}
				break;
			}
		}
	}

	if (AchievementData.IsEmpty())
	{
		Delegate.ExecuteIfBound(PlayerId, bAllSucceeded);
	}
	else
	{
		auto QueryTask = new FOnlineAsyncTaskGooglePlayWriteAchievements(Subsystem, PlayerId.AsShared(), MoveTemp(AchievementData), Delegate);
		Subsystem->QueueAsyncTask(QueryTask);
	}
}

void FOnlineAchievementsGooglePlay::WriteAchievements( const FUniqueNetId& PlayerId, FOnlineAchievementsWriteRef& WriteObject, const FOnAchievementsWrittenDelegate& Delegate )
{
	if (!IsLocalPlayer(PlayerId))
	{
		UE_LOG_ONLINE_ACHIEVEMENTS(Warning, TEXT("WriteAchievements failed because was called using non local player or local player is not logged in"));
		Delegate.ExecuteIfBound(PlayerId, false);
		return;
	}
		
	if (Achievements.IsEmpty())
	{
		auto QueryTask = new FOnlineAsyncTaskGooglePlayQueryAchievements(Subsystem,
			PlayerId.AsShared(),
			FOnQueryAchievementsCompleteDelegate::CreateRaw(
				this,
				&FOnlineAchievementsGooglePlay::FinishAchievementWrite,
				WriteObject,
				Delegate));
		Subsystem->QueueAsyncTask(QueryTask);
	}
	else
	{
		FinishAchievementWrite(PlayerId, true, WriteObject, Delegate);
	}
}

EOnlineCachedResult::Type FOnlineAchievementsGooglePlay::GetCachedAchievements(const FUniqueNetId& PlayerId, TArray<FOnlineAchievement>& OutAchievements)
{
	if (!IsLocalPlayer(PlayerId))
	{
		UE_LOG_ONLINE_ACHIEVEMENTS(Warning, TEXT("GetCachedAchievements failed because was called using non local player or local player is not logged in"));
		return EOnlineCachedResult::NotFound;
	}

	OutAchievements.Empty();

	if (Achievements.IsEmpty())
	{
		return EOnlineCachedResult::NotFound;
	}

	for (auto& CurrentAchievement : Achievements)
	{
		OutAchievements.Add(CurrentAchievement);
	}

	return EOnlineCachedResult::Success;
}


EOnlineCachedResult::Type FOnlineAchievementsGooglePlay::GetCachedAchievementDescription(const FString& AchievementId, FOnlineAchievementDesc& OutAchievementDesc)
{
	FOnlineAchievementDesc* AchievementDescription = AchievementDescriptions.Find(AchievementId);
	if (!AchievementDescription)
	{
		return EOnlineCachedResult::NotFound;
	}

	OutAchievementDesc = *AchievementDescription;

	return EOnlineCachedResult::Success;
}

#if !UE_BUILD_SHIPPING
bool FOnlineAchievementsGooglePlay::ResetAchievements( const FUniqueNetId& PlayerId )
{
	// Not supported client side. Use Management APIs for this: https://developers.google.com/games/services/management/api/achievements/reset
	return false;
};
#endif // !UE_BUILD_SHIPPING

EOnlineCachedResult::Type FOnlineAchievementsGooglePlay::GetCachedAchievement( const FUniqueNetId& PlayerId, const FString& AchievementId, FOnlineAchievement& OutAchievement )
{
	if (!IsLocalPlayer(PlayerId))
	{
		UE_LOG_ONLINE_ACHIEVEMENTS(Warning, TEXT("GetCachedAchievement failed because was called using non local player or local player is not logged in"));
		return EOnlineCachedResult::NotFound;
	}

	FOnlineAchievement* FoundAchievement = Achievements.FindByPredicate([AchievementId](const FOnlineAchievement& Achievement) { return Achievement.Id == AchievementId; });
	if (!FoundAchievement)
	{
		return EOnlineCachedResult::NotFound;
	}

	OutAchievement = *FoundAchievement;

	return EOnlineCachedResult::Success;
}

void FOnlineAchievementsGooglePlay::UpdateCache(TArray<FOnlineAchievementGooglePlay>&& Results, TArray<FOnlineAchievementDesc>&& Descriptions)
{
	ensure(Results.Num() == Descriptions.Num());

	int32 Count = FPlatformMath::Min(Results.Num(), Descriptions.Num());
	AchievementDescriptions.Reset();
	Achievements.Reset(Count);

	auto Settings = GetDefault<UAndroidRuntimeSettings>();

	for (int32 Idx = 0; Idx < Count; ++Idx)
	{
		FOnlineAchievementGooglePlay& Achievement = Results[Idx];
		if (TOptional<FString> Id = GetUnrealAchievementIdFromGoogleAchievementId(Settings, Achievement.Id))
		{ 
			Achievement.Id = MoveTemp(*Id);
			AchievementDescriptions.Emplace(Achievement.Id, MoveTemp(Descriptions[Idx]));
			Achievements.Add(MoveTemp(Achievement));
		}
	}
}

void FOnlineAchievementsGooglePlay::UpdateCacheAfterWrite(const TArray<FGooglePlayAchievementWriteData>& WrittenData)
{
	auto Settings = GetDefault<UAndroidRuntimeSettings>();

	for (const FGooglePlayAchievementWriteData& Data: WrittenData)
	{
		FOnlineAchievementGooglePlay* Achievement = nullptr;

		if (TOptional<FString> UnrealAchievementId = GetUnrealAchievementIdFromGoogleAchievementId(Settings, Data.GooglePlayAchievementId))
		{
			Achievement = Achievements.FindByPredicate([&UnrealAchievementId](const FOnlineAchievementGooglePlay& Achievement)
				{
					return Achievement.Id == *UnrealAchievementId;
				});
		}

		if (!Achievement)
		{
            UE_LOG_ONLINE_ACHIEVEMENTS(Error, TEXT("Unknown Google achievement id: %s"), *Data.GooglePlayAchievementId);
			continue;
		}

		switch (Data.Action)
		{
			case FGooglePlayAchievementWriteAction::WriteSteps:
			{
				float NewProgress = (float)( (100. * Data.Steps) / Achievement->TotalSteps);
				if (NewProgress > Achievement->Progress)
				{ 
					Achievement->Progress = NewProgress;
				}
				break;
			}
			case FGooglePlayAchievementWriteAction::Unlock:
			{
				Achievement->Progress = 100.f;
				break;
			}
		}
	}
}

void FOnlineAchievementsGooglePlay::ClearCache()
{
	Achievements.Empty();
}
