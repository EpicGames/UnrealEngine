// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "OnlineSubsystemTypes.h"
#include "OnlineAchievementGooglePlayCommon.h"
#include "OnlineSubsystemGooglePlayPackage.h"

/**
 *	IOnlineAchievements - Interface class for Achievements
 */
class FOnlineAchievementsGooglePlay : public IOnlineAchievements
{
private:

	/** Reference to the main subsystem */
	class FOnlineSubsystemGooglePlay* Subsystem;
	
	/** Cached achievements retrieved from Google Play */
	TArray<FOnlineAchievementGooglePlay> Achievements;

	/** Cached achievement descriptions retrieved from Google Play */
	TMap<FString, FOnlineAchievementDesc> AchievementDescriptions;

	static TOptional<FString> GetUnrealAchievementIdFromGoogleAchievementId(const class UAndroidRuntimeSettings* Settings, const FString& GoogleId);
	static TOptional<FString> GetGoogleAchievementIdFromUnrealAchievementId(const class UAndroidRuntimeSettings* Settings, const FString& UnrealId);

	/**
	 * Using the WriteObject, fires off achievement progress calls to the Google Java backend. Non-blocking.
	 *
	 * @param PlayerId the id of the player who's making progress
	 * @param bWasSuccessful whether a previous QueryAchievements call was successful
	 * @param WriteObject achievement write object provided by the user
	 * @param Delegate delegate to execute when the write operation is finished
	 */
	void FinishAchievementWrite(
		const FUniqueNetId& PlayerId,
		const bool bWasSuccessful,
		FOnlineAchievementsWriteRef WriteObject,
		FOnAchievementsWrittenDelegate Delegate);
	
	/** Asks the identity interface it this PlayerId refers to the local player. We can only get information about the local player from GooglePlay */
	bool IsLocalPlayer(const FUniqueNetId& PlayerId) const;
PACKAGE_SCOPE:
	/** Clears the cache of Google achievements that was populated by a QueryAchievements() call. */
	void ClearCache();

	/** Called from the query achievements task to fill in the cache. */
	void UpdateCache(TArray<FOnlineAchievementGooglePlay>&& Results, TArray<FOnlineAchievementDesc>&& Descriptions);

	/** Called from the write achievements task to update cache with written data. */
	void UpdateCacheAfterWrite(const TArray<FGooglePlayAchievementWriteData>& WrittenData);
public:

	//~ Begin IOnlineAchievements Interface
	virtual void WriteAchievements(const FUniqueNetId& PlayerId, FOnlineAchievementsWriteRef& WriteObject, const FOnAchievementsWrittenDelegate& Delegate = FOnAchievementsWrittenDelegate()) override;
	virtual void QueryAchievements(const FUniqueNetId& PlayerId, const FOnQueryAchievementsCompleteDelegate& Delegate = FOnQueryAchievementsCompleteDelegate()) override;
	virtual void QueryAchievementDescriptions(const FUniqueNetId& PlayerId, const FOnQueryAchievementsCompleteDelegate& Delegate = FOnQueryAchievementsCompleteDelegate()) override ;
	virtual EOnlineCachedResult::Type GetCachedAchievement(const FUniqueNetId& PlayerId, const FString& AchievementId, FOnlineAchievement& OutAchievement) override;
	virtual EOnlineCachedResult::Type GetCachedAchievements(const FUniqueNetId& PlayerId, TArray<FOnlineAchievement>& OutAchievements) override;
	virtual EOnlineCachedResult::Type GetCachedAchievementDescription(const FString& AchievementId, FOnlineAchievementDesc& OutAchievementDesc) override;
#if !UE_BUILD_SHIPPING
	virtual bool ResetAchievements( const FUniqueNetId& PlayerId ) override;
#endif // !UE_BUILD_SHIPPING
	//~ End IOnlineAchievements Interface

	/**
	 * Constructor
	 *
	 * @param InSubsystem - A reference to the owning subsystem
	 */
	FOnlineAchievementsGooglePlay( class FOnlineSubsystemGooglePlay* InSubsystem );
};

typedef TSharedPtr<FOnlineAchievementsGooglePlay, ESPMode::ThreadSafe> FOnlineAchievementsGooglePlayPtr;
