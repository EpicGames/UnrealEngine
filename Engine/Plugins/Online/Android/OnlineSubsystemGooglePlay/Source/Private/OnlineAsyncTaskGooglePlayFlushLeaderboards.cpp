// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineAsyncTaskGooglePlayFlushLeaderboards.h"
#include "GooglePlayGamesWrapper.h"
#include "OnlineLeaderboardInterfaceGooglePlay.h"
#include "OnlineSubsystemGooglePlay.h"

FOnlineAsyncTaskGooglePlayFlushLeaderboards::FOnlineAsyncTaskGooglePlayFlushLeaderboards(
	FOnlineSubsystemGooglePlay* InSubsystem,
	FName InSessionName,
	TArray<FGooglePlayLeaderboardScore>&& InLeaderboardScores)
	: FOnlineAsyncTaskBasic(InSubsystem)
	, SessionName(InSessionName)
	, LeaderboardScores(MoveTemp(InLeaderboardScores))
{
}

void FOnlineAsyncTaskGooglePlayFlushLeaderboards::Tick()
{
	if ( !bStarted)
	{
		bStarted = true;
		bWasSuccessful = Subsystem->GetGooglePlayGamesWrapper().FlushLeaderboardsScores(this, MoveTemp(LeaderboardScores)); 
		bIsComplete = !bWasSuccessful;
	}
}

void FOnlineAsyncTaskGooglePlayFlushLeaderboards::TriggerDelegates()
{
	Subsystem->GetLeaderboardsGooglePlay()->TriggerOnLeaderboardFlushCompleteDelegates(SessionName, bWasSuccessful);
}

