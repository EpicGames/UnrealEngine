// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "OnlineAsyncTaskManager.h"
#include "OnlineLeaderboardGooglePlayCommon.h"

class FOnlineSubsystemGooglePlay;

/** Task object to keep track of writing achievement score data 
 * Objects of this type are associated with a GooglePlayGamesWrapper method call that routes operations to the Java backend. 
 * The GooglePlayGamesWrapper implementation will adapt and set the task result and mark the task as completed when the 
 * Java operation is finished 
 */
class FOnlineAsyncTaskGooglePlayFlushLeaderboards : public FOnlineAsyncTaskBasic<FOnlineSubsystemGooglePlay>
{
public:
	/**
	 * @brief Construct a new FOnlineAsyncTaskGooglePlayFlushLeaderboards object
	 * 
	 * @param InSubsystem Owning subsystem 
	 * @param InSessionName Session name to report using in the delegate
	 * @param LeaderboardScores Achievement data to submit 
	 */
	FOnlineAsyncTaskGooglePlayFlushLeaderboards( 
		FOnlineSubsystemGooglePlay* InSubsystem, 
		FName InSessionName, 
		TArray<FGooglePlayLeaderboardScore>&& LeaderboardScores);

	// FOnlineAsyncTask
	virtual void Tick() override;
	virtual void TriggerDelegates() override;

	// FOnlineAsyncItem
	virtual FString ToString() const override { return TEXT("FlushLeaderboards"); }

private:
	TArray<FGooglePlayLeaderboardScore> LeaderboardScores;
	FName SessionName;
	bool bStarted = false;
};
