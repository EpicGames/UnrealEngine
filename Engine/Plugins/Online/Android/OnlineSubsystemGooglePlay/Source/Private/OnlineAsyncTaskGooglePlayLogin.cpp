// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineAsyncTaskGooglePlayLogin.h"
#include "GooglePlayGamesWrapper.h"
#include "OnlineSubsystemGooglePlay.h"

FOnlineAsyncTaskGooglePlayLogin::FOnlineAsyncTaskGooglePlayLogin(FOnlineSubsystemGooglePlay* InSubsystem, const FString& InAuthCodeClientId, bool InForceRefreshToken)
	: FOnlineAsyncTaskBasic(InSubsystem)
	, AuthCodeClientId(InAuthCodeClientId)
	, ForceRefreshToken(InForceRefreshToken)
	, ReceivedPlayerNetId(FUniqueNetIdGooglePlay::EmptyId())
{
}

void FOnlineAsyncTaskGooglePlayLogin::Tick()
{
	if ( !bStarted)
	{
		bStarted = true;
		bWasSuccessful = Subsystem->GetGooglePlayGamesWrapper().Login(this, AuthCodeClientId, ForceRefreshToken); 
		bIsComplete = !bWasSuccessful;
	}
}

void FOnlineAsyncTaskGooglePlayLogin::Finalize()
{
	FOnlineIdentityGooglePlayPtr IdentityInt = Subsystem->GetIdentityGooglePlay();
	FUniqueNetIdPtr CurrentNetId = IdentityInt->GetUniquePlayerId(0);
	bWasLoggedIn = CurrentNetId && CurrentNetId->IsValid();
	if (bWasSuccessful)
	{
		IdentityInt->SetIdentityData(ReceivedPlayerNetId, MoveTemp(ReceivedDisplayName), MoveTemp(ReceivedAuthCode));
	}
	else
	{
		IdentityInt->ClearIdentity();
	}
}

void FOnlineAsyncTaskGooglePlayLogin::TriggerDelegates()
{
	FOnlineIdentityGooglePlayPtr IdentityInt = Subsystem->GetIdentityGooglePlay();
	IdentityInt->TriggerOnLoginCompleteDelegates(0, bWasSuccessful, *ReceivedPlayerNetId, TEXT(""));
	if (!bWasLoggedIn)
	{
		IdentityInt->TriggerOnLoginStatusChangedDelegates(0, ELoginStatus::NotLoggedIn, ELoginStatus::LoggedIn, *ReceivedPlayerNetId);
		IdentityInt->TriggerOnLoginChangedDelegates(0);
	}
}

void FOnlineAsyncTaskGooglePlayLogin::SetLoginData(FString&& PlayerId, FString&& DisplayName, FString&& AuthCode)
{
	ReceivedPlayerNetId = FUniqueNetIdGooglePlay::Create(MoveTemp(PlayerId));
	ReceivedDisplayName = MoveTemp(DisplayName);
	ReceivedAuthCode = MoveTemp(AuthCode);
}
