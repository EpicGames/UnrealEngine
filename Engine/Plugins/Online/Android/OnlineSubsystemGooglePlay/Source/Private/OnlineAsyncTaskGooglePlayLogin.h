// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "OnlineAsyncTaskManager.h"
#include "OnlineIdentityInterfaceGooglePlay.h"

class FOnlineSubsystemGooglePlay;

/** Task object to keep track of writing achievement score data 
 * Objects of this type are associated with a GooglePlayGamesWrapper method call that routes operations to the Java backend. 
 * The GooglePlayGamesWrapper implementation will adapt and set the task result and mark the task as completed when the 
 * Java operation is finished 
 */
class FOnlineAsyncTaskGooglePlayLogin : public FOnlineAsyncTaskBasic<FOnlineSubsystemGooglePlay>
{
public:
	/**
	 * @brief Construct a new FOnlineAsyncTaskGooglePlayLogin object
	 * 
	 * @param InSubsystem Owning subsystem
	 * @param InAuthCodeClientId The client ID of the server that will perform the authorization code flow exchange. Won't request authorization code if empty
	 * @param InForceRefreshToken If true a refresh token will be included in addition to the access token when the authorization code is exchanged
	 */
	FOnlineAsyncTaskGooglePlayLogin(
		FOnlineSubsystemGooglePlay* InSubsystem, 
		const FString& InAuthCodeClientId, 
		bool InForceRefreshToken);

	// FOnlineAsyncTask
	virtual void Tick() override;
	void Finalize() override;
	void TriggerDelegates() override;

	// FOnlineAsyncItem
	virtual FString ToString() const override { return TEXT("Login"); }

	// Set task result data. Accessed trhough GooglePlayGamesWrapper implementation
	void SetLoginData(FString&& PlayerId, FString&& PlayerDisplayName, FString&& AuthCode);
private:
	FString AuthCodeClientId;
	bool ForceRefreshToken;
	bool bStarted = false;
	bool bWasLoggedIn = false;
	FUniqueNetIdGooglePlayPtr ReceivedPlayerNetId;
	FString ReceivedDisplayName;
	FString ReceivedAuthCode;
};
