// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineAsyncTaskGooglePlayQueryAchievements.h"
#include "OnlineSubsystemGooglePlay.h"
#include "OnlineAchievementsInterfaceGooglePlay.h"

FOnlineAsyncTaskGooglePlayQueryAchievements::FOnlineAsyncTaskGooglePlayQueryAchievements(
	FOnlineSubsystemGooglePlay* InSubsystem,
	const FUniqueNetIdPtr& InPlayerId,
	const FOnQueryAchievementsCompleteDelegate& InDelegate)
	: FOnlineAsyncTaskBasic(InSubsystem)
	, PlayerId(InPlayerId)
	, Delegate(InDelegate)
{
}

void FOnlineAsyncTaskGooglePlayQueryAchievements::SetAchievementsData(TArray<FOnlineAchievementGooglePlay>&& InAchievementsData, TArray<FOnlineAchievementDesc>&& InAchievementsDesc)
{
	AchievementsData = MoveTemp(InAchievementsData);	
	AchievementsDesc = MoveTemp(InAchievementsDesc);
}

void FOnlineAsyncTaskGooglePlayQueryAchievements::Tick()
{
	if ( !bStarted)
	{
		bStarted = true;
		bWasSuccessful = Subsystem->GetGooglePlayGamesWrapper().QueryAchievements(this); 
		bIsComplete = !bWasSuccessful;
	}
}

void FOnlineAsyncTaskGooglePlayQueryAchievements::Finalize()
{
	FOnlineAchievementsGooglePlayPtr AchievementsInt = Subsystem->GetAchievementsGooglePlay();

	if (bWasSuccessful)
	{
		AchievementsInt->UpdateCache(MoveTemp(AchievementsData), MoveTemp(AchievementsDesc));
	}
	else
	{
		AchievementsInt->ClearCache();
	}
}

void FOnlineAsyncTaskGooglePlayQueryAchievements::TriggerDelegates()
{
	Delegate.ExecuteIfBound(*PlayerId, bWasSuccessful);
}