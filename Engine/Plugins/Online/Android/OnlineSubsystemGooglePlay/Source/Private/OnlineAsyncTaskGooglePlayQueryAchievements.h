// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "OnlineAsyncTaskManager.h"
#include "OnlineAchievementGooglePlayCommon.h"
#include "OnlineSubsystemGooglePlayPackage.h"

class FOnlineSubsystemGooglePlay;

/** Task object to keep track of writing achievement score data 
 * Objects of this type are associated with a GooglePlayGamesWrapper method call that routes operations to the Java backend. 
 * The GooglePlayGamesWrapper implementation will adapt and set the task result and mark the task as completed when the 
 * Java operation is finished 
 */
class FOnlineAsyncTaskGooglePlayQueryAchievements : public FOnlineAsyncTaskBasic<FOnlineSubsystemGooglePlay>
{
public:
	/**
	 * @brief Construct a new FOnlineAsyncTaskGooglePlayQueryAchievements object
	 * 
	 * @param InSubsystem Owning subsystem
	 * @param PlayerId PlayerId to report in the delegate. Only data for local player can be received from GooglePlay
	 * @param InDelegate Delegate to invoke on task completion
	 */
	FOnlineAsyncTaskGooglePlayQueryAchievements(
		FOnlineSubsystemGooglePlay* InSubsystem,
		const FUniqueNetIdPtr& PlayerId,
		const FOnQueryAchievementsCompleteDelegate& InDelegate);

	// FOnlineAsyncTask
	virtual void Tick() override;
	virtual void Finalize() override;
	virtual void TriggerDelegates() override;

	// FOnlineAsyncItem
	virtual FString ToString() const override { return TEXT("QueryAchievements"); }

	// Set task result data. Accessed trhough GooglePlayGamesWrapper implementation
	void SetAchievementsData(TArray<FOnlineAchievementGooglePlay>&& InAchievementsData, TArray<FOnlineAchievementDesc>&& InAchievementsDesc);
private:
	FUniqueNetIdPtr PlayerId;
	FOnQueryAchievementsCompleteDelegate Delegate;
	TArray<FOnlineAchievementGooglePlay> AchievementsData;
	TArray<FOnlineAchievementDesc> AchievementsDesc;
	bool bStarted = false;
};
