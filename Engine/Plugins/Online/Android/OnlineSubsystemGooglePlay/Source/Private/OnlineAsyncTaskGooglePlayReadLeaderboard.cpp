// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineAsyncTaskGooglePlayReadLeaderboard.h"
#include "GooglePlayGamesWrapper.h"
#include "OnlineLeaderboardInterfaceGooglePlay.h"
#include "OnlineSubsystemGooglePlay.h"

FOnlineAsyncTaskGooglePlayReadLeaderboard::FOnlineAsyncTaskGooglePlayReadLeaderboard(
	FOnlineSubsystemGooglePlay* InSubsystem,
	const FOnlineLeaderboardReadRef& InReadObject,
	const FString& InLeaderboardId )
	: FOnlineAsyncTaskBasic(InSubsystem)
	, ReadObject(InReadObject)
	, LeaderboardId(InLeaderboardId)
{
	ReadObject->Rows.Empty();
	ReadObject->ReadState = EOnlineAsyncTaskState::InProgress;
}

void FOnlineAsyncTaskGooglePlayReadLeaderboard::Tick()
{
	if ( !bStarted)
	{
		bStarted = true;
		bWasSuccessful = Subsystem->GetGooglePlayGamesWrapper().RequestLeaderboardScore(this, LeaderboardId); 
		bIsComplete = !bWasSuccessful;
	}
}

void FOnlineAsyncTaskGooglePlayReadLeaderboard::Finalize()
{
	ReadObject->ReadState = bWasSuccessful? EOnlineAsyncTaskState::Done : EOnlineAsyncTaskState::Failed;
}

void FOnlineAsyncTaskGooglePlayReadLeaderboard::TriggerDelegates()
{
	Subsystem->GetLeaderboardsGooglePlay()->TriggerOnLeaderboardReadCompleteDelegates(bWasSuccessful);
}

void FOnlineAsyncTaskGooglePlayReadLeaderboard::AddScore(const FString& DisplayName, const FString& PlayerId, int64 Rank, int64 RawScore)
{
	if (!PlayerId.IsEmpty())
	{
		FUniqueNetIdRef UserId = FUniqueNetIdGooglePlay::Create(PlayerId);
		FOnlineStatsRow* UserRow = ReadObject->FindPlayerRecord(*UserId);
		if (UserRow == NULL)
		{
			UserRow = &ReadObject->Rows.Emplace_GetRef(DisplayName, UserId);
		}

		UserRow->Rank = Rank;

		for (const FColumnMetaData& ColumnMeta: ReadObject->ColumnMetadata)
		{
			switch (ColumnMeta.DataType)
			{
				case EOnlineKeyValuePairDataType::Int64:
				{
					UserRow->Columns.Add(ColumnMeta.ColumnName, FVariantData(RawScore));
					break;
				}
				case EOnlineKeyValuePairDataType::Int32:
				{
					int32 Value = (int32)FMath::Clamp(RawScore, MIN_int32, MAX_int32);
					UserRow->Columns.Add(ColumnMeta.ColumnName, FVariantData(Value));
					break;
				}

				default:
				{
					UE_LOG_ONLINE(Warning, TEXT("Unsupported key value pair during retrieval from Google Play %s"), *ColumnMeta.ColumnName);
					break;
				}
			}
		}
	}}