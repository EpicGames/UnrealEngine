// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "OnlineStats.h"
#include "OnlineAsyncTaskManager.h"

class FOnlineSubsystemGooglePlay;

/** Task object to keep track of writing achievement score data 
 * Objects of this type are associated with a GooglePlayGamesWrapper method call that routes operations to the Java backend. 
 * The GooglePlayGamesWrapper implementation will adapt and set the task result and mark the task as completed when the 
 * Java operation is finished 
 */
class FOnlineAsyncTaskGooglePlayReadLeaderboard : public FOnlineAsyncTaskBasic<FOnlineSubsystemGooglePlay>
{
public:
	/**
	 * @brief Construct a new FOnlineAsyncTaskGooglePlayReadLeaderboard object
	 * 
	 * @param InSubsystem Owning subsystem 
	 * @param InReadObject Destination object for the leaderboard information received
	 * @param InLeaderboardId GooglePlay leaderboard id to request
	 */
	FOnlineAsyncTaskGooglePlayReadLeaderboard( 
		FOnlineSubsystemGooglePlay* InSubsystem, 
		const FOnlineLeaderboardReadRef& InReadObject, 
		const FString& InLeaderboardId);

	// FOnlineAsyncTask
	virtual void Tick() override;
	void Finalize() override;
	void TriggerDelegates() override;

	// FOnlineAsyncItem
	virtual FString ToString() const override { return TEXT("ReadLeaderboard"); }

	// Set task result data. Accessed trhough GooglePlayGamesWrapper implementation
	void AddScore(const FString& DisplayName, const FString& PlayerId, int64 Rank, int64 RawScore);
private:
	FString LeaderboardId;
	bool bStarted = false;
	FOnlineLeaderboardReadRef ReadObject;
};
