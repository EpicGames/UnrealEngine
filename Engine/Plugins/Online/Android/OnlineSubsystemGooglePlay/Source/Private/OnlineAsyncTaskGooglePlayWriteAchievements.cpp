// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineAsyncTaskGooglePlayWriteAchievements.h"
#include "Algo/NoneOf.h"
#include "OnlineSubsystemGooglePlay.h"
#include "OnlineAchievementsInterfaceGooglePlay.h"

FOnlineAsyncTaskGooglePlayWriteAchievements::FOnlineAsyncTaskGooglePlayWriteAchievements(
	FOnlineSubsystemGooglePlay* InSubsystem,
	const FUniqueNetIdPtr& InPlayerId,
	TArray<FGooglePlayAchievementWriteData>&& InWriteAchievements,
	const FOnAchievementsWrittenDelegate& InDelegate)
	: FOnlineAsyncTaskBasic(InSubsystem)
	, PlayerId(InPlayerId)
	, WriteAchievementsData(MoveTemp(InWriteAchievements))
	, Delegate(InDelegate)
{
}

void FOnlineAsyncTaskGooglePlayWriteAchievements::Tick()
{
	if ( !bStarted)
	{
		bStarted = true;
		bWasSuccessful = Subsystem->GetGooglePlayGamesWrapper().WriteAchievements(this, WriteAchievementsData); 
		bIsComplete = !bWasSuccessful;
	}
}

bool FOnlineAsyncTaskGooglePlayWriteAchievements::SetSucceeded(const TArray<FString>& SucceededIds)
{
	// Remove data of entries that did not succeed. Task succeeded if no entry was removed
	return 0 == WriteAchievementsData.RemoveAll([&SucceededIds](const FGooglePlayAchievementWriteData& Entry)
		{
			return Algo::NoneOf(SucceededIds, [&Entry](const FString& Id) { return Id == Entry.GooglePlayAchievementId;} );
		});
}

void FOnlineAsyncTaskGooglePlayWriteAchievements::Finalize()
{
	FOnlineAchievementsGooglePlayPtr AchievementsInt = Subsystem->GetAchievementsGooglePlay();
	AchievementsInt->UpdateCacheAfterWrite(WriteAchievementsData);
}

void FOnlineAsyncTaskGooglePlayWriteAchievements::TriggerDelegates()
{
	Delegate.ExecuteIfBound(*PlayerId, bWasSuccessful);
}