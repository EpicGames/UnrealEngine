// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "OnlineAsyncTaskManager.h"
#include "OnlineAchievementGooglePlayCommon.h"
#include "Interfaces/OnlineAchievementsInterface.h"

class FOnlineSubsystemGooglePlay;

/** Task object to keep track of writing achievement score data 
 * Objects of this type are associated with a GooglePlayGamesWrapper method call that routes operations to the Java backend. 
 * The GooglePlayGamesWrapper implementation will adapt and set the task result and mark the task as completed when the 
 * Java operation is finished 
 */
class FOnlineAsyncTaskGooglePlayWriteAchievements : public FOnlineAsyncTaskBasic<FOnlineSubsystemGooglePlay>
{
public:
	/**
	 * @brief Construct a new FOnlineAsyncTaskGooglePlayWriteAchievements object
	 * 
	 * @param InSubsystem Owning subsystem
	 * @param PlayerId PlayerId to report in the delegate. Only data for local player can be received from GooglePlay
	 * @param InWriteAchievements Achievement data to send
	 * @param Delegate Delegate to invoke on task completion
	 */
	FOnlineAsyncTaskGooglePlayWriteAchievements(
		FOnlineSubsystemGooglePlay* InSubsystem,
		const FUniqueNetIdPtr& PlayerId,
		TArray<FGooglePlayAchievementWriteData>&& InWriteAchievements,
		const FOnAchievementsWrittenDelegate& Delegate);

	/**
	 * @brief Notifies the list of achievement writes that succeeded
	 * 
	 * @param SucceededIds 
	 * @return true If all achievements expected to be written succeeded
	 * @return false Otherwise
	 */
	bool SetSucceeded(const TArray<FString>& SucceededIds);

	// FOnlineAsyncTask
	virtual void Tick() override;
	virtual void Finalize() override;
	virtual void TriggerDelegates() override;

	// FOnlineAsyncItem
	virtual FString ToString() const override { return TEXT("QueryAchievements"); }

private:
	FUniqueNetIdPtr PlayerId;
	TArray<FGooglePlayAchievementWriteData> WriteAchievementsData;
	FOnAchievementsWrittenDelegate Delegate;
	bool bStarted = false;
};
