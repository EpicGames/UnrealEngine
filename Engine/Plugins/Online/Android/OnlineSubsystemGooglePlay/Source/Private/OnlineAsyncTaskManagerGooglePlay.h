// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "OnlineAsyncTaskManager.h"
#include "OnlineSubsystemGooglePlayPackage.h"

/**
 *	Google Play version of the async task manager
 */
class FOnlineAsyncTaskManagerGooglePlay : public FOnlineAsyncTaskManager
{
public:

	FOnlineAsyncTaskManagerGooglePlay() {}

	// FOnlineAsyncTaskManager
	virtual void OnlineTick() override {}
};


