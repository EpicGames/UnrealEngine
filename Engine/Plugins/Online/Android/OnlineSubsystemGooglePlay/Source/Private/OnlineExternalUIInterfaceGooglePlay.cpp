// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineExternalUIInterfaceGooglePlay.h"
#include "AndroidRuntimeSettings.h"
#include "UObject/Class.h"
#include "OnlineSubsystemGooglePlay.h"

FOnlineExternalUIGooglePlay::FOnlineExternalUIGooglePlay(FOnlineSubsystemGooglePlay* InSubsystem)
	: Subsystem(InSubsystem)
{
	check(Subsystem != nullptr);
}

bool FOnlineExternalUIGooglePlay::ShowLoginUI(const int ControllerIndex, bool bShowOnlineOnly, bool bShowSkipButton, const FOnLoginUIClosedDelegate& Delegate)
{
	IOnlineIdentityPtr OnlineIdentity = Subsystem->GetIdentityInterface();

	if (FUniqueNetIdPtr UniqueNetId = OnlineIdentity->GetUniquePlayerId(ControllerIndex); UniqueNetId->IsValid())
	{
		Delegate.ExecuteIfBound(UniqueNetId, ControllerIndex, FOnlineError::Success());
		return true;
	}

	TSharedPtr<FDelegateHandle> DelegateHandle = MakeShared<FDelegateHandle>();
	*DelegateHandle = OnlineIdentity->AddOnLoginCompleteDelegate_Handle(ControllerIndex, FOnLoginCompleteDelegate::CreateLambda([this, DelegateHandle, Delegate](int32 LocalUserNum, bool bWasSuccessful, const FUniqueNetId& UserId, const FString& ErrorString)
		{
			FOnlineError Error(bWasSuccessful);
			Error.SetFromErrorCode(ErrorString);

			Delegate.ExecuteIfBound(UserId.IsValid()? UserId.AsShared() : FUniqueNetIdPtr(), LocalUserNum, Error);

			IOnlineIdentityPtr OnlineIdentity = Subsystem->GetIdentityInterface();
			OnlineIdentity->ClearOnLoginCompleteDelegate_Handle(LocalUserNum, *DelegateHandle);
		}));
	OnlineIdentity->Login(ControllerIndex, FOnlineAccountCredentials());
	return true;
}

bool FOnlineExternalUIGooglePlay::ShowFriendsUI(int32 LocalUserNum)
{
	return false;
}

bool FOnlineExternalUIGooglePlay::ShowInviteUI(int32 LocalUserNum, FName SessionName)
{
	return false;
}

bool FOnlineExternalUIGooglePlay::ShowAchievementsUI(int32 LocalUserNum) 
{
	Subsystem->GetGooglePlayGamesWrapper().ShowAchievementsUI();
	return true;
}

bool FOnlineExternalUIGooglePlay::ShowLeaderboardUI(const FString& LeaderboardName)
{
	auto Settings = GetDefault<UAndroidRuntimeSettings>();

	for(const auto& Mapping : Settings->LeaderboardMap)
	{
		if(Mapping.Name == LeaderboardName)
		{
			Subsystem->GetGooglePlayGamesWrapper().ShowLeaderboardUI(Mapping.LeaderboardID);
			return true;
		}
	}
	return true;
}

bool FOnlineExternalUIGooglePlay::ShowWebURL(const FString& Url, const FShowWebUrlParams& ShowParams, const FOnShowWebUrlClosedDelegate& Delegate)
{
	FPlatformProcess::LaunchURL(*Url, nullptr, nullptr);

	return true;
}

bool FOnlineExternalUIGooglePlay::CloseWebURL()
{
	return false;
}

bool FOnlineExternalUIGooglePlay::ShowProfileUI( const FUniqueNetId& Requestor, const FUniqueNetId& Requestee, const FOnProfileUIClosedDelegate& Delegate )
{
	return false;
}

bool FOnlineExternalUIGooglePlay::ShowAccountUpgradeUI(const FUniqueNetId& UniqueId)
{
	return false;
}

bool FOnlineExternalUIGooglePlay::ShowStoreUI(int32 LocalUserNum, const FShowStoreParams& ShowParams, const FOnShowStoreUIClosedDelegate& Delegate)
{
	return false;
}

bool FOnlineExternalUIGooglePlay::ShowSendMessageUI(int32 LocalUserNum, const FShowSendMessageParams& ShowParams, const FOnShowSendMessageUIClosedDelegate& Delegate)
{
	return false;
}