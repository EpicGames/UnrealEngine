// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "OnlineDelegateMacros.h"
#include "Interfaces/OnlineExternalUIInterface.h"
#include "OnlineSubsystemGooglePlayPackage.h"

class FOnlineSubsystemGooglePlay;

/** 
 * Interface definition for the online services external UIs
 * Any online service that provides extra UI overlays will implement the relevant functions
 */
class FOnlineExternalUIGooglePlay :
	public IOnlineExternalUI
{
private:
	FOnlineSubsystemGooglePlay* Subsystem;

public:
	FOnlineExternalUIGooglePlay(FOnlineSubsystemGooglePlay* InSubsystem);

	virtual bool ShowLoginUI(const int ControllerIndex, bool bShowOnlineOnly, bool bShowSkipButton, const FOnLoginUIClosedDelegate& Delegate = FOnLoginUIClosedDelegate()) override;
	virtual bool ShowAccountCreationUI(const int ControllerIndex, const FOnAccountCreationUIClosedDelegate& Delegate = FOnAccountCreationUIClosedDelegate()) override { /** NYI */ return false; }
	virtual bool ShowFriendsUI(int32 LocalUserNum) override;
	virtual bool ShowInviteUI(int32 LocalUserNum, FName SessionName) override;
	virtual bool ShowAchievementsUI(int32 LocalUserNum) override;
	virtual bool ShowLeaderboardUI(const FString& LeaderboardName) override;
	virtual bool ShowWebURL(const FString& Url, const FShowWebUrlParams& ShowParams, const FOnShowWebUrlClosedDelegate& Delegate = FOnShowWebUrlClosedDelegate()) override;
	virtual bool CloseWebURL() override;
	virtual bool ShowProfileUI(const FUniqueNetId& Requestor, const FUniqueNetId& Requestee, const FOnProfileUIClosedDelegate& Delegate = FOnProfileUIClosedDelegate()) override;
	virtual bool ShowAccountUpgradeUI(const FUniqueNetId& UniqueId) override;
	virtual bool ShowStoreUI(int32 LocalUserNum, const FShowStoreParams& ShowParams, const FOnShowStoreUIClosedDelegate& Delegate = FOnShowStoreUIClosedDelegate()) override;
	virtual bool ShowSendMessageUI(int32 LocalUserNum, const FShowSendMessageParams& ShowParams, const FOnShowSendMessageUIClosedDelegate& Delegate = FOnShowSendMessageUIClosedDelegate()) override;
};

typedef TSharedPtr<FOnlineExternalUIGooglePlay, ESPMode::ThreadSafe> FOnlineExternalUIGooglePlayPtr;

