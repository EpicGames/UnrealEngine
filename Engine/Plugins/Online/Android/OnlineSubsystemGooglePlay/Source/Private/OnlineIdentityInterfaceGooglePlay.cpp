// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineIdentityInterfaceGooglePlay.h"
#include "AndroidRuntimeSettings.h"
#include "OnlineAsyncTaskGooglePlayLogin.h"
#include "OnlineSubsystemGooglePlay.h"

class FUserOnlineAccountGooglePlay : public FUserOnlineAccount
{
public:
	FUserOnlineAccountGooglePlay(const FUniqueNetIdGooglePlayRef& InPlayerNetId, FString InPlayerAlias, FString InAuthCode)
		: UniqueNetId(InPlayerNetId)
		, PlayerAlias(MoveTemp(InPlayerAlias))
		, AuthCode(MoveTemp(InAuthCode))
	{}

	// FOnlineUser
	virtual FUniqueNetIdRef GetUserId() const override 
	{ 
		return UniqueNetId; 
	}
	
	virtual FString GetRealName() const override 
	{ 
		return FString(); 
	}
	
	virtual FString GetDisplayName(const FString& Platform = FString()) const override 
	{ 
		return PlayerAlias; 
	}

	virtual bool GetUserAttribute(const FString& AttrName, FString& OutAttrValue) const override 
	{ 
		if (AttrName == USER_ATTR_DISPLAYNAME)
		{
			OutAttrValue = PlayerAlias;
			return true;
		}
		else if (AttrName == USER_ATTR_ID)
		{
			OutAttrValue = UniqueNetId->ToString();
            return true;
		}
		return false;
	}
	
	virtual bool SetUserLocalAttribute(const FString& AttrName, const FString& InAttrValue) override 
	{ 
		return false;
	}

	// FUserOnlineAccount
	virtual FString GetAccessToken() const override 
	{ 
		return FString(); 
	}
	
	virtual bool HasAccessTokenExpired(const FDateTime& Time) const override 
	{ 
		return true; 
	}

	virtual bool GetAuthAttribute(const FString& AttrName, FString& OutAttrValue) const override
	{
		if (AttrName == AUTH_ATTR_AUTHORIZATION_CODE && !AuthCode.IsEmpty())
		{
			OutAttrValue = AuthCode;
			return true;
		}
		return false;
	}
	
	virtual bool SetUserAttribute(const FString& AttrName, const FString& AttrValue) override 
	{ 
		return false; 
	}
private:
	FUniqueNetIdGooglePlayRef UniqueNetId;
	FString PlayerAlias;
	FString AuthCode;
};

FOnlineIdentityGooglePlay::FOnlineIdentityGooglePlay(FOnlineSubsystemGooglePlay* InSubsystem)
	: MainSubsystem(InSubsystem)
{
	UE_LOG_ONLINE_IDENTITY(Display, TEXT("FOnlineIdentityGooglePlay::FOnlineIdentityGooglePlay()"));
	check(MainSubsystem != nullptr);

	ClearIdentity();
}

void FOnlineIdentityGooglePlay::ClearIdentity()
{
	LocalPlayerAccount.Reset();
}

TSharedPtr<FUserOnlineAccount> FOnlineIdentityGooglePlay::GetUserAccount(const FUniqueNetId& UserId) const
{
	if (LocalPlayerAccount && *LocalPlayerAccount->GetUserId() == UserId)
	{
		return LocalPlayerAccount;
	}
	return nullptr;
}

TArray<TSharedPtr<FUserOnlineAccount> > FOnlineIdentityGooglePlay::GetAllUserAccounts() const
{
	TArray<TSharedPtr<FUserOnlineAccount> > Result;
	if (LocalPlayerAccount)
	{
		Result.Add(LocalPlayerAccount);
	}

	return Result;
}

bool FOnlineIdentityGooglePlay::Login(int32 LocalUserNum, const FOnlineAccountCredentials& AccountCredentials) 
{
	if (LocalUserNum > 0)
	{
		TriggerOnLoginCompleteDelegates(LocalUserNum, false, *FUniqueNetIdGooglePlay::EmptyId(), FString("FOnlineIdentityGooglePlay does not support more than 1 local player"));
		return false;
	}
	
	if (LocalPlayerAccount.IsValid())
	{
		TriggerOnLoginCompleteDelegates(0, true, *LocalPlayerAccount->GetUserId(), TEXT(""));
		return false;
	}

	auto Settings = GetDefault<UAndroidRuntimeSettings>();

	MainSubsystem->QueueAsyncTask(new FOnlineAsyncTaskGooglePlayLogin(MainSubsystem, Settings->PlayGamesClientId, Settings->bForceRefreshToken));
	
	return true;
}

void FOnlineIdentityGooglePlay::SetIdentityData(const FUniqueNetIdGooglePlayPtr& InPlayerNetId, FString InPlayerAlias, FString InAuthCode)
{
	LocalPlayerAccount = MakeShared<FUserOnlineAccountGooglePlay>(InPlayerNetId.ToSharedRef(), MoveTemp(InPlayerAlias), MoveTemp(InAuthCode));
}

bool FOnlineIdentityGooglePlay::Logout(int32 LocalUserNum)
{
	if (LocalUserNum == 0)
	{
		bool bWasLoggedIn = LocalPlayerAccount.IsValid();
		ClearIdentity();
		if(bWasLoggedIn)
		{
			TriggerOnLoginStatusChangedDelegates(0, ELoginStatus::LoggedIn, ELoginStatus::NotLoggedIn, *FUniqueNetIdGooglePlay::EmptyId());
			TriggerOnLoginChangedDelegates(0);
		}
	}
	return false;
}

bool FOnlineIdentityGooglePlay::AutoLogin(int32 LocalUserNum)
{
	return Login(LocalUserNum, FOnlineAccountCredentials());
}

ELoginStatus::Type FOnlineIdentityGooglePlay::GetLoginStatus(int32 LocalUserNum) const
{
	if (LocalUserNum == 0 && LocalPlayerAccount.IsValid())
	{
		return  ELoginStatus::LoggedIn;
	}
	else
	{
		return ELoginStatus::NotLoggedIn;
	}
}

ELoginStatus::Type FOnlineIdentityGooglePlay::GetLoginStatus(const FUniqueNetId& UserId) const
{
	if (UserId.IsValid() && LocalPlayerAccount && *LocalPlayerAccount->GetUserId() == UserId)
	{
		return ELoginStatus::LoggedIn;
	}
	else
	{
		return ELoginStatus::NotLoggedIn;
	}
}

FUniqueNetIdPtr FOnlineIdentityGooglePlay::GetUniquePlayerId(int32 LocalUserNum) const
{
	if (LocalUserNum == 0 && LocalPlayerAccount)
	{
		return LocalPlayerAccount->GetUserId();
	}

	return FUniqueNetIdGooglePlay::EmptyId();
}

FUniqueNetIdPtr FOnlineIdentityGooglePlay::CreateUniquePlayerId(uint8* Bytes, int32 Size)
{
	if( Bytes && Size == sizeof(uint64) )
	{
		int32 StrLen = FCString::Strlen((TCHAR*)Bytes);
		if (StrLen > 0)
		{
			FString StrId((TCHAR*)Bytes);
			return FUniqueNetIdGooglePlay::Create(StrId);
		}
	}
	return NULL;
}

FUniqueNetIdPtr FOnlineIdentityGooglePlay::CreateUniquePlayerId(const FString& Str)
{
	return FUniqueNetIdGooglePlay::Create(Str);
}

FString FOnlineIdentityGooglePlay::GetPlayerNickname(int32 LocalUserNum) const
{
	if (LocalUserNum == 0 && LocalPlayerAccount)
	{ 
		return LocalPlayerAccount->GetDisplayName();
	}
	else
	{
		return FString();
	}
}

FString FOnlineIdentityGooglePlay::GetPlayerNickname(const FUniqueNetId& UserId) const
{
	if (UserId.IsValid() && LocalPlayerAccount && *LocalPlayerAccount->GetUserId() == UserId)
	{
		return GetPlayerNickname(0);
	}
	else
	{
		return FString();
	}
}

FString FOnlineIdentityGooglePlay::GetAuthToken(int32 LocalUserNum) const
{
	if (LocalUserNum == 0 && LocalPlayerAccount)
	{
		return LocalPlayerAccount->GetAccessToken();
	}
	return FString();
}

void FOnlineIdentityGooglePlay::RevokeAuthToken(const FUniqueNetId& UserId, const FOnRevokeAuthTokenCompleteDelegate& Delegate)
{
	UE_LOG_ONLINE_IDENTITY(Display, TEXT("FOnlineIdentityGooglePlay::RevokeAuthToken not implemented"));
	FUniqueNetIdRef UserIdRef(UserId.AsShared());
	MainSubsystem->ExecuteNextTick([UserIdRef, Delegate]()
	{
		Delegate.ExecuteIfBound(*UserIdRef, FOnlineError(FString(TEXT("RevokeAuthToken not implemented"))));
	});
}

void FOnlineIdentityGooglePlay::GetUserPrivilege(const FUniqueNetId& UserId, EUserPrivileges::Type Privilege, const FOnGetUserPrivilegeCompleteDelegate& Delegate, EShowPrivilegeResolveUI ShowResolveUI)
{
	Delegate.ExecuteIfBound(UserId, Privilege, (uint32)EPrivilegeResults::NoFailures);
}

FPlatformUserId FOnlineIdentityGooglePlay::GetPlatformUserIdFromUniqueNetId(const FUniqueNetId& NetId) const
{
	for (int i = 0; i < MAX_LOCAL_PLAYERS; ++i)
	{
		auto CurrentUniqueId = GetUniquePlayerId(i);
		if (CurrentUniqueId.IsValid() && (*CurrentUniqueId == NetId))
		{
			return GetPlatformUserIdFromLocalUserNum(i);
		}
	}

	return PLATFORMUSERID_NONE;
}

FString FOnlineIdentityGooglePlay::GetAuthType() const
{
	return TEXT("");
}
