// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Interfaces/OnlineIdentityInterface.h"
#include "OnlineSubsystemGooglePlayPackage.h"

// from OnlineSubsystemTypes.h
TEMP_UNIQUENETIDSTRING_SUBCLASS(FUniqueNetIdGooglePlay, GOOGLEPLAY_SUBSYSTEM);

class FOnlineIdentityGooglePlay :
	public IOnlineIdentity
{
private:
	TSharedPtr<class FUserOnlineAccountGooglePlay> LocalPlayerAccount;

	class FOnlineSubsystemGooglePlay* MainSubsystem;

    static_assert(MAX_LOCAL_PLAYERS == 1, "FOnlineIdentityGooglePlay does not support more than 1 local player");

PACKAGE_SCOPE:

	FOnlineIdentityGooglePlay(FOnlineSubsystemGooglePlay* InSubsystem);

	void SetIdentityData(const FUniqueNetIdGooglePlayPtr& PlayerNetId, FString PlayerAlias, FString AuthCode);
	void ClearIdentity();

public:

	//~ Begin IOnlineIdentity Interface
	virtual TSharedPtr<FUserOnlineAccount> GetUserAccount(const FUniqueNetId& UserId) const override;
	virtual TArray<TSharedPtr<FUserOnlineAccount> > GetAllUserAccounts() const override;
	virtual bool Login(int32 LocalUserNum, const FOnlineAccountCredentials& AccountCredentials) override;
	virtual bool Logout(int32 LocalUserNum) override;
	virtual bool AutoLogin(int32 LocalUserNum) override;
	virtual FUniqueNetIdPtr GetUniquePlayerId(int32 LocalUserNum) const override;
	virtual FUniqueNetIdPtr CreateUniquePlayerId(uint8* Bytes, int32 Size) override;
	virtual FUniqueNetIdPtr CreateUniquePlayerId(const FString& Str) override;
	virtual ELoginStatus::Type GetLoginStatus(int32 LocalUserNum) const override;
	virtual ELoginStatus::Type GetLoginStatus(const FUniqueNetId& UserId) const override;
	virtual FString GetPlayerNickname(int32 LocalUserNum) const override;
	virtual FString GetPlayerNickname(const FUniqueNetId& UserId) const override;
	virtual FString GetAuthToken(int32 LocalUserNum) const override;
	virtual void RevokeAuthToken(const FUniqueNetId& UserId, const FOnRevokeAuthTokenCompleteDelegate& Delegate) override;
	virtual void GetUserPrivilege(const FUniqueNetId& UserId, EUserPrivileges::Type Privilege, const FOnGetUserPrivilegeCompleteDelegate& Delegate, EShowPrivilegeResolveUI ShowResolveUI=EShowPrivilegeResolveUI::Default) override;
	virtual FPlatformUserId GetPlatformUserIdFromUniqueNetId(const FUniqueNetId& NetId) const override;
	virtual FString GetAuthType() const override;
	//~ End IOnlineIdentity Interface	
};


typedef TSharedPtr<FOnlineIdentityGooglePlay, ESPMode::ThreadSafe> FOnlineIdentityGooglePlayPtr;

