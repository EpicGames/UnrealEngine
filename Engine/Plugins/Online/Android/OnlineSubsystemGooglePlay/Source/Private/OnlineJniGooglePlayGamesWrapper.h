// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
 
#include "CoreMinimal.h"
#if USE_ANDROID_JNI
#include "Android/AndroidJavaEnv.h"

namespace UE::Jni
{
	struct FGooglePlayGamesWrapper: Java::Lang::FObject
	{
		static constexpr bool bIsOptional = true;
		
		static constexpr FAnsiStringView ClassName = "com/epicgames/unreal/GooglePlayGamesWrapper";

		static void JNICALL nativeLoginSuccess(JNIEnv* env, jclass clazz, jlong TaskPtr, jstring InPlayerId, jstring InPlayerDisplayName, jstring InAuthCode);
		static void JNICALL nativeLoginFailed(JNIEnv* env, jclass clazz, jlong TaskPtr);
		static void JNICALL nativeLeaderboardRequestSuccess(JNIEnv* env, jclass clazz, jlong TaskPtr, jstring InDisplayName, jstring InPlayerId, jlong InRank, jlong InRawScore);
		static void JNICALL nativeLeaderboardRequestFailed(JNIEnv* env, jclass clazz, jlong TaskPtr);
		static void JNICALL nativeFlushLeaderboardsCompleted(JNIEnv* env, jclass clazz, jlong TaskPtr, jboolean InSuccess);
		static void JNICALL nativeQueryAchievementsSuccess(JNIEnv* env, jclass clazz, jlong TaskPtr, Java::Lang::TArray<jstring>* InAchievementIds, Java::Lang::TArray<jint>* InTypes, Java::Lang::TArray<jint>* InSteps, Java::Lang::TArray<jint>* InTotalSteps, Java::Lang::TArray<jstring>* InTitles, Java::Lang::TArray<jstring>* InDescriptions, Java::Lang::TArray<jboolean>* InIsHidden, Java::Lang::TArray<jlong>* InLastUpdatedTimestamps);
		static void JNICALL nativeQueryAchievementsFailed(JNIEnv* env, jclass clazz, jlong TaskPtr);
		static void JNICALL nativeWriteAchievementsCompleted(JNIEnv* env, jclass clazz, jlong TaskPtr, Java::Lang::TArray<jstring>* InSucceeded);
		
		static constexpr FNativeMethod NativeMethods[]
		{
			UE_JNI_NATIVE_METHOD(nativeLoginSuccess),
			UE_JNI_NATIVE_METHOD(nativeLoginFailed),
			UE_JNI_NATIVE_METHOD(nativeLeaderboardRequestSuccess),
			UE_JNI_NATIVE_METHOD(nativeLeaderboardRequestFailed),
			UE_JNI_NATIVE_METHOD(nativeFlushLeaderboardsCompleted),
			UE_JNI_NATIVE_METHOD(nativeQueryAchievementsSuccess),
			UE_JNI_NATIVE_METHOD(nativeQueryAchievementsFailed),
			UE_JNI_NATIVE_METHOD(nativeWriteAchievementsCompleted)
		};	
	};

	template struct TInitialize<FGooglePlayGamesWrapper>;
}
#endif
