// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#if USE_ANDROID_JNI
#include "Android/AndroidJavaEnv.h"

namespace UE::Jni
{
	struct FGooglePlayStoreHelper: Java::Lang::FObject
	{
		static constexpr bool bIsOptional = true;
		
		static constexpr FAnsiStringView ClassName = "com/epicgames/unreal/GooglePlayStoreHelper";

		static void JNICALL NativeQueryComplete(JNIEnv* env, jobject thiz, jint ResponseCode, Java::Lang::TArray<jstring>* ProductIDs, Java::Lang::TArray<jstring>* Titles, Java::Lang::TArray<jstring>* Descriptions, Java::Lang::TArray<jstring>* Prices, Java::Lang::TArray<jlong>* PriceValuesRaw, Java::Lang::TArray<jstring>* CurrencyCodes, Java::Lang::TArray<jstring>* RegularPrices, Java::Lang::TArray<jlong>* RegularPriceValuesRaw, Java::Lang::TArray<jint>* DiscountTypes);
		static void JNICALL NativeQueryExistingPurchasesComplete(JNIEnv* env, jobject thiz, jint ResponseCode, Java::Lang::TArray<jstring, 2>* ProductIDs, Java::Lang::TArray<jint>* PurchaseStatesArray, Java::Lang::TArray<jstring>* PurchaseTokens, Java::Lang::TArray<jstring>* ReceiptsData, Java::Lang::TArray<jstring>* Signatures);
		static void JNICALL NativePurchaseComplete(JNIEnv* env, jobject thiz, jint JavaResponseCode, Java::Lang::TArray<jstring>* JavaProductIds, jint JavaPurchaseState, jstring JavaPurchaseToken, jstring JavaReceiptData, jstring JavaSignature);
		static void JNICALL NativeConsumeComplete(JNIEnv* env, jobject thiz, jint JavaResponseCode, jstring JavaPurchaseToken);
		static void JNICALL NativeAcknowledgeComplete(JNIEnv* env, jobject thiz, jint JavaResponseCode, jstring JavaPurchaseToken);
		static void JNICALL NativeAlternativeBillingOnlyInformationDialogComplete(JNIEnv* env, jobject thiz, jint JavaResponseCode);

		static constexpr FNativeMethod NativeMethods[]
		{
			UE_JNI_NATIVE_METHOD(NativeQueryComplete),
			UE_JNI_NATIVE_METHOD(NativeQueryExistingPurchasesComplete),
			UE_JNI_NATIVE_METHOD(NativePurchaseComplete),
			UE_JNI_NATIVE_METHOD(NativeConsumeComplete),
			UE_JNI_NATIVE_METHOD(NativeAcknowledgeComplete),
			UE_JNI_NATIVE_METHOD(NativeAlternativeBillingOnlyInformationDialogComplete)
		};
	};
	
	template struct TInitialize<FGooglePlayStoreHelper>;
}
#endif
