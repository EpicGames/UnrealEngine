// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"

struct FGooglePlayLeaderboardScore
{
	// Leaderboard ID as shown in GooglePlay Console
	FString GooglePlayLeaderboardId;
	// Leaderboard score
	int64 Score = 0;
};
