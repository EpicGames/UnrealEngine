// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineLeaderboardInterfaceGooglePlay.h"
#include "Algo/NoneOf.h"
#include "AndroidRuntimeSettings.h"
#include "OnlineAchievementGooglePlayCommon.h"
#include "OnlineAsyncTaskGooglePlayReadLeaderboard.h"
#include "OnlineAsyncTaskGooglePlayFlushLeaderboards.h"
#include "OnlineSubsystemGooglePlay.h"

namespace LeaderboardsGooglePlayDetail
{
	static TOptional<FString> GetGooglePlayLeaderboardId(const UAndroidRuntimeSettings* Settings, const FString& LeaderboardName)
	{
		for(const auto& Mapping : Settings->LeaderboardMap)
		{
			if(Mapping.Name == LeaderboardName)
			{
				return Mapping.LeaderboardID;
			}
		}

		UE_LOG_ONLINE_LEADERBOARD(Warning, TEXT( "GetGooglePlayLeaderboardId: No mapping for leaderboard %s"), *LeaderboardName );
		return NullOpt;
	}
}

FOnlineLeaderboardsGooglePlay::FOnlineLeaderboardsGooglePlay(FOnlineSubsystemGooglePlay* InSubsystem)
	: Subsystem(InSubsystem)
{
	check(Subsystem);
}

bool FOnlineLeaderboardsGooglePlay::IsLocalPlayer(const FUniqueNetId& PlayerId) const
{
	if (!PlayerId.IsValid())
	{
		return false;
	}
	FOnlineIdentityGooglePlayPtr IdentityInt = Subsystem->GetIdentityGooglePlay();
	FUniqueNetIdPtr LocalPlayerNetId = IdentityInt->GetUniquePlayerId(0);
	return LocalPlayerNetId? *LocalPlayerNetId == PlayerId : false;
}

bool FOnlineLeaderboardsGooglePlay::ReadLeaderboards(const TArray< FUniqueNetIdRef >& Players, FOnlineLeaderboardReadRef& ReadObject)
{
	if (Algo::NoneOf(Players, [this](const FUniqueNetIdRef& PlayerId) { return IsLocalPlayer(*PlayerId); }))
	{
		UE_LOG_ONLINE_LEADERBOARD(Warning, TEXT("ReadLeaderboards failed because was called using non local players or local player is not logged in"));
		return false;
	}

	ReadObject->Rows.Empty();

	auto Settings = GetDefault<UAndroidRuntimeSettings>();
	if(TOptional<FString> PlatformLeaderboardId = LeaderboardsGooglePlayDetail::GetGooglePlayLeaderboardId(Settings, ReadObject->LeaderboardName))
	{
		Subsystem->QueueAsyncTask(new FOnlineAsyncTaskGooglePlayReadLeaderboard( Subsystem, ReadObject, *PlatformLeaderboardId));
		return true;
	}
	return false;
}

bool FOnlineLeaderboardsGooglePlay::ReadLeaderboardsForFriends(int32 LocalUserNum, FOnlineLeaderboardReadRef& ReadObject)
{
	UE_LOG_ONLINE_LEADERBOARD(Warning, TEXT("ReadLeaderboardsForFriends is not supported on Google Play."));
	TriggerOnLeaderboardReadCompleteDelegates(false);
	return false;
}

void FOnlineLeaderboardsGooglePlay::FreeStats(FOnlineLeaderboardRead& ReadObject)
{
	// iOS doesn't support this, and there is no Google Play functionality for this either
}

bool FOnlineLeaderboardsGooglePlay::WriteLeaderboards(const FName& SessionName, const FUniqueNetId& Player, FOnlineLeaderboardWrite& WriteObject)
{
	if (!IsLocalPlayer(Player))
	{
		UE_LOG_ONLINE_LEADERBOARD(Warning, TEXT("WriteLeaderboards failed because was called using non local player or local player is not logged in"));
		return false;
	}

	bool bWroteAnyLeaderboard = false;
	auto Settings = GetDefault<UAndroidRuntimeSettings>();

	for(int32 LeaderboardIdx = 0; LeaderboardIdx < WriteObject.LeaderboardNames.Num(); ++LeaderboardIdx)
	{
		FString LeaderboardName = WriteObject.LeaderboardNames[LeaderboardIdx];
		UE_LOG_ONLINE_LEADERBOARD(Display, TEXT("Going through stats for leaderboard :  %s "), *LeaderboardName);
		
		for(auto &[Key, Stat]: WriteObject.Properties)
		{
			TOptional<FString> GooglePlayLeaderboardId = LeaderboardsGooglePlayDetail::GetGooglePlayLeaderboardId(Settings, LeaderboardName);
			if (!GooglePlayLeaderboardId)
			{
				continue;
			}

			int64 Score = 0;

			switch(Stat.GetType())
			{
				case EOnlineKeyValuePairDataType::Int64:
				{
					Stat.GetValue(Score);
					break;
				}
				case EOnlineKeyValuePairDataType::Int32:
				{
					// cast from 32 to 64
					int32 Score32 = 0;
					Stat.GetValue(Score32);
					Score = Score32;
					break;
				}
				default:
				{
					UE_LOG_ONLINE_LEADERBOARD(Warning, TEXT("Unsupported stat type %s"), EOnlineKeyValuePairDataType::ToString(Stat.GetType()));
					continue;
				}
			}
			UE_LOG_ONLINE_LEADERBOARD(Display, TEXT("FOnlineLeaderboardsGooglePlay::WriteLeaderboards() %s value Score: %s"), EOnlineKeyValuePairDataType::ToString(Stat.GetType()), *Stat.ToString());
			FGooglePlayLeaderboardScore& UnreportedScore = UnreportedScores.Emplace_GetRef();
			UnreportedScore.GooglePlayLeaderboardId = MoveTemp(*GooglePlayLeaderboardId);
			UnreportedScore.Score = Score;
			bWroteAnyLeaderboard = true;
		}
	}
	
	//Return whether any stat was cached
	return bWroteAnyLeaderboard;
}

bool FOnlineLeaderboardsGooglePlay::FlushLeaderboards(const FName& SessionName)
{
	Subsystem->QueueAsyncTask(new FOnlineAsyncTaskGooglePlayFlushLeaderboards( Subsystem, SessionName, MoveTemp(UnreportedScores)));
	UnreportedScores.Empty();
	return true;
}

bool FOnlineLeaderboardsGooglePlay::ReadLeaderboardsAroundRank(int32 Rank, uint32 Range, FOnlineLeaderboardReadRef& ReadObject)
{
	UE_LOG_ONLINE_LEADERBOARD(Warning, TEXT("FOnlineLeaderboardsGooglePlay::ReadLeaderboardsAroundRank is currently not supported."));
	return false;
}
bool FOnlineLeaderboardsGooglePlay::ReadLeaderboardsAroundUser(FUniqueNetIdRef Player, uint32 Range, FOnlineLeaderboardReadRef& ReadObject)
{
	UE_LOG_ONLINE_LEADERBOARD(Warning, TEXT("FOnlineLeaderboardsGooglePlay::ReadLeaderboardsAroundUser is currently not supported."));
	return false;
}

bool FOnlineLeaderboardsGooglePlay::WriteOnlinePlayerRatings(const FName& SessionName, int32 LeaderboardId, const TArray<FOnlinePlayerScore>& PlayerScores)
{
	//iOS doesn't support this, and there is no Google Play functionality for this either
	return false;
}