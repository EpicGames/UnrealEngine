// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Interfaces/OnlineLeaderboardInterface.h"
#include "OnlineLeaderboardGooglePlayCommon.h"
#include "OnlineSubsystemGooglePlayPackage.h"

/**
 * Interface definition for the online services leaderboard services 
 */
class FOnlineLeaderboardsGooglePlay : public IOnlineLeaderboards
{
public:
	FOnlineLeaderboardsGooglePlay(class FOnlineSubsystemGooglePlay* InSubsystem);

	virtual bool ReadLeaderboards(const TArray< FUniqueNetIdRef >& Players, FOnlineLeaderboardReadRef& ReadObject) override;
	virtual bool ReadLeaderboardsForFriends(int32 LocalUserNum, FOnlineLeaderboardReadRef& ReadObject) override;
	virtual bool ReadLeaderboardsAroundRank(int32 Rank, uint32 Range, FOnlineLeaderboardReadRef& ReadObject) override;
	virtual bool ReadLeaderboardsAroundUser(FUniqueNetIdRef Player, uint32 Range, FOnlineLeaderboardReadRef& ReadObject) override;
	virtual void FreeStats(FOnlineLeaderboardRead& ReadObject) override;
	virtual bool WriteLeaderboards(const FName& SessionName, const FUniqueNetId& Player, FOnlineLeaderboardWrite& WriteObject) override;
	virtual bool FlushLeaderboards(const FName& SessionName) override;
	virtual bool WriteOnlinePlayerRatings(const FName& SessionName, int32 LeaderboardId, const TArray<FOnlinePlayerScore>& PlayerScores) override;

private:
	/** Pointer to owning subsystem */
	FOnlineSubsystemGooglePlay* Subsystem;

	FString GetLeaderboardID(const FString& LeaderboardName);

	/** Scores are cached here in WriteLeaderboards until FlushLeaderboards is called */
	TArray<FGooglePlayLeaderboardScore> UnreportedScores;

	/** Asks the identity interface it this PlayerId refers to the local player. We can only get information about the local player from GooglePlay */
	bool IsLocalPlayer(const FUniqueNetId& PlayerId) const;
};

typedef TSharedPtr<FOnlineLeaderboardsGooglePlay, ESPMode::ThreadSafe> FOnlineLeaderboardsGooglePlayPtr;
