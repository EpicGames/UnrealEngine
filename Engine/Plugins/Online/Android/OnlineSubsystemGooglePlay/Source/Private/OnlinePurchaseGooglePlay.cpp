// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlinePurchaseGooglePlay.h"
#include "OnlineJniGooglePlayStoreHelper.h"
#include "OnlineSubsystemGooglePlay.h"
#include "OnlineError.h"
#include "Misc/ConfigCacheIni.h"
#include "Algo/AllOf.h"

#define LOCTEXT_NAMESPACE "OnlineSubsystemGooglePlay"

FGoogleTransactionData::FGoogleTransactionData(const TArray<FString>& InOfferIds, const FString& InPurchaseToken, const FString& InReceiptData, const FString& InSignature, EGooglePlayPurchaseState InPurchaseState)
	: OfferIds(InOfferIds)
	, PurchaseToken(InPurchaseToken)
	, CombinedTransactionData(InOfferIds, InReceiptData, InSignature)
	, PurchaseState(InPurchaseState)
{
	if (PurchaseToken.IsEmpty())
	{
		ErrorStr = TEXT("Receipt does not contain purchase token");
	}
	else if (CombinedTransactionData.ReceiptData.IsEmpty())
	{
		ErrorStr = TEXT("Receipt does not contain receipt data");
	}
	else if (CombinedTransactionData.Signature.IsEmpty())
	{
		ErrorStr = TEXT("Receipt does not contain signature data");
	}
}

FString FGoogleTransactionData::ToDebugString() const
{
	return FString::Printf(TEXT("OfferIds: %s PurchaseState: %s TransactionId: %s ReceiptData: %s%s"),
		*FString::Join(OfferIds, TEXT(",")),
		LexToString(PurchaseState),
		*PurchaseToken,
		*CombinedTransactionData.ToJson(),
		ErrorStr.IsEmpty() ? TEXT("") : *FString::Printf(TEXT(" Error: %s"), *ErrorStr));
}

bool FGoogleTransactionData::IsMatchingRequest(const FPurchaseCheckoutRequest& Request) const
{
	return Algo::AllOf(GetOfferIds(), [&Request](const FString& OfferId) {
				return Request.PurchaseOffers.ContainsByPredicate([&OfferId](const FPurchaseCheckoutRequest::FPurchaseOfferEntry& Entry) 
				{
					FStringView CheckedOutProductId(Entry.OfferId);
					int32 ColonPos = INDEX_NONE;
					if (CheckedOutProductId.FindChar(TEXT(':'), ColonPos))
					{
						CheckedOutProductId = CheckedOutProductId.Left(ColonPos);
					}
					return (OfferId == CheckedOutProductId);
				});
		   }); 
}

FGoogleTransactionData::FJsonReceiptData::FJsonReceiptData(const TArray<FString>& InOfferIds, const FString& InReceiptData, const FString& InSignature)
	: ReceiptData(InReceiptData)
	, Signature(InSignature)
{ 
	if (!InOfferIds.IsEmpty() && FOnlinePurchaseGooglePlay::IsSubscriptionProductId(InOfferIds[0]))
	{
		IsSubscription = true;
	}
}

/**
 * Info used to cache and track the order being currently purchased.
 */
class FOnlinePurchaseGooglePlay::FOnlinePurchaseInProgressTransaction
{
public:
	FOnlinePurchaseInProgressTransaction(
		const FPurchaseCheckoutRequest& InCheckoutRequest,
		const FUniqueNetId& InUserId,
		const EPurchaseTransactionState InPendingTransactionState,
		const FOnPurchaseCheckoutComplete& InCheckoutCompleteDelegate
		);
	
	/**
	 * Generate a final receipt for all purchases made in this single transaction
	 */
	TSharedRef<FPurchaseReceipt> GenerateReceipt();
	
	/** Generate one off receipts for transactions initiated outside the current run of the application */
	static TSharedRef<FPurchaseReceipt> GenerateReceipt(const FGoogleTransactionData& Transaction);
	
	/** Add the single completed transaction to this transaction */
	bool AddCompletedOffer(EPurchaseTransactionState Result, const FGoogleTransactionData& Transaction);

	static void FillReceiptContent(FPurchaseReceipt& Receipt, const FGoogleTransactionData& Transaction);

public:
	
	/** Checkout info for the in progress order */
	FPurchaseCheckoutRequest CheckoutRequest;
	/** User for the order in progress */
	FUniqueNetIdGooglePlayRef UserId;
	/** Delegate to call on completion */
	FOnPurchaseCheckoutComplete CheckoutCompleteDelegate;
	/** Tracks the current state of the order */
	FPurchaseReceipt PurchaseInfo;
};


FOnlinePurchaseGooglePlay::FOnlinePurchaseInProgressTransaction::FOnlinePurchaseInProgressTransaction(
	const FPurchaseCheckoutRequest& InCheckoutRequest,
	const FUniqueNetId& InUserId,
	const EPurchaseTransactionState InPendingTransactionState,
	const FOnPurchaseCheckoutComplete& InCheckoutCompleteDelegate
	)
	: CheckoutRequest(InCheckoutRequest)
	, UserId(FUniqueNetIdGooglePlay::Cast(InUserId.AsShared()))
	, CheckoutCompleteDelegate(InCheckoutCompleteDelegate)
{
	PurchaseInfo.TransactionState = InPendingTransactionState;
}

TSharedRef<FPurchaseReceipt> FOnlinePurchaseGooglePlay::FOnlinePurchaseInProgressTransaction::GenerateReceipt()
{
	TSharedRef<FPurchaseReceipt> Receipt = MakeShared<FPurchaseReceipt>();
	
	Receipt->TransactionState = PurchaseInfo.TransactionState;
	Receipt->TransactionId = PurchaseInfo.TransactionId;
	
	if(PurchaseInfo.TransactionState == EPurchaseTransactionState::Purchased ||
	   PurchaseInfo.TransactionState == EPurchaseTransactionState::Deferred)
	{
		Receipt->ReceiptOffers = PurchaseInfo.ReceiptOffers;
	}
	else
	{
		// Add the requested offers to the receipt in the event of an incomplete purchase.
		for(const auto& RequestedOffer : CheckoutRequest.PurchaseOffers)
		{
			Receipt->AddReceiptOffer(RequestedOffer.OfferNamespace, RequestedOffer.OfferId, RequestedOffer.Quantity);
		}
	}
	
	return Receipt;
}

void FOnlinePurchaseGooglePlay::FOnlinePurchaseInProgressTransaction::FillReceiptContent(FPurchaseReceipt& Receipt, const FGoogleTransactionData& Transaction)
{
	// Pending purchases should not be redeemable although they already have a transaction identifier
	const bool ShouldAddValidationInfo = (Transaction.GetPurchaseState() == EGooglePlayPurchaseState::Purchased);
	for(const FString& OfferId: Transaction.GetOfferIds())
	{
		FPurchaseReceipt::FReceiptOfferEntry ReceiptEntry(TEXT(""), OfferId, 1);
		FPurchaseReceipt::FLineItemInfo& LineItem = ReceiptEntry.LineItems.Emplace_GetRef();
		
		LineItem.ItemName = OfferId;
		LineItem.UniqueId = Transaction.GetTransactionIdentifier();
		if(ShouldAddValidationInfo)
		{
			LineItem.ValidationInfo = Transaction.GetCombinedReceiptData();
		}
		Receipt.AddReceiptOffer(MoveTemp(ReceiptEntry));
	}
}

TSharedRef<FPurchaseReceipt> FOnlinePurchaseGooglePlay::FOnlinePurchaseInProgressTransaction::GenerateReceipt(const FGoogleTransactionData& Transaction)
{
	TSharedRef<FPurchaseReceipt> Receipt = MakeShared<FPurchaseReceipt>();
	
	Receipt->TransactionState = Transaction.GetErrorStr().IsEmpty() ? EPurchaseTransactionState::Purchased : EPurchaseTransactionState::Failed;
	Receipt->TransactionId = Transaction.GetTransactionIdentifier();
	
	if (Receipt->TransactionState == EPurchaseTransactionState::Purchased)
	{
		FillReceiptContent(*Receipt, Transaction);
	}
	
	return Receipt;
}

bool FOnlinePurchaseGooglePlay::FOnlinePurchaseInProgressTransaction::AddCompletedOffer(EPurchaseTransactionState Result, const FGoogleTransactionData& Transaction)
{
	bool bIsFromThisRequest = Transaction.IsMatchingRequest(CheckoutRequest);
	
	if(!bIsFromThisRequest)
	{
		return false;
	}

	FillReceiptContent(PurchaseInfo, Transaction);
	return true;
}

FOnlinePurchaseGooglePlay::FOnlinePurchaseGooglePlay(FOnlineSubsystemGooglePlay* InSubsystem)
	: Subsystem(InSubsystem)
{
	UE_LOG_ONLINE_PURCHASE(Verbose, TEXT( "FOnlinePurchaseGooglePlay::FOnlinePurchaseGooglePlay" ));
}

FOnlinePurchaseGooglePlay::~FOnlinePurchaseGooglePlay()
{
}

void FOnlinePurchaseGooglePlay::Init()
{
	UE_LOG_ONLINE_PURCHASE(Verbose, TEXT("FOnlinePurchaseGooglePlay::Init"));

	FString GooglePlayLicenseKey;
	if (!GConfig->GetString(TEXT("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings"), TEXT("GooglePlayLicenseKey"), GooglePlayLicenseKey, GEngineIni) || GooglePlayLicenseKey.IsEmpty())
	{
		UE_LOG_ONLINE_STOREV2(Warning, TEXT("Missing GooglePlayLicenseKey key in /Script/AndroidRuntimeSettings.AndroidRuntimeSettings of DefaultEngine.ini"));
	}

	GConfig->GetBool(TEXT("OnlineSubsystemGooglePlay.Store"), TEXT("bDisableLocalAcknowledgeAndConsume"), bDisableLocalAcknowledgeAndConsume, GEngineIni);

	bool EnableAlternativeBillingOnly = false;
	GConfig->GetBool(TEXT("OnlineSubsystemGooglePlay.Store"), TEXT("EnableAlternativeBillingOnly"), EnableAlternativeBillingOnly, GEngineIni);

	extern void AndroidThunkCpp_Iap_SetupIapService(const FString&, bool);
	AndroidThunkCpp_Iap_SetupIapService(GooglePlayLicenseKey, EnableAlternativeBillingOnly);
}

bool FOnlinePurchaseGooglePlay::IsSubscriptionProductId(const FString& ProductId)
{
	return ProductId.StartsWith(TEXT("s-"));
}

bool FOnlinePurchaseGooglePlay::IsAllowedToPurchase(const FUniqueNetId& UserId)
{
	return true;
}

bool FOnlinePurchaseGooglePlay::IsAllowedToPurchase(const FUniqueNetId& UserId, FOnlineError& Error)
{
	Error =  FOnlineError(EOnlineErrorResult::Success);
	return true;
}

bool FOnlinePurchaseGooglePlay::IsBillingClientConnected()
{
	extern bool AndroidThunkCpp_Iap_IsAllowedToMakePurchases();
	bool bIsBillingClientConnected = AndroidThunkCpp_Iap_IsAllowedToMakePurchases();
	UE_LOG_ONLINE_PURCHASE(Verbose, TEXT("FOnlinePurchaseGooglePlay::IsBillingClientConnected: %d"), bIsBillingClientConnected);
	return bIsBillingClientConnected;
}

void FOnlinePurchaseGooglePlay::Checkout(const FUniqueNetId& UserId, const FPurchaseCheckoutRequest& CheckoutRequest, const FOnPurchaseCheckoutComplete& Delegate)
{
	bool bStarted = false;
	FText ErrorMessage;

	if (InProgressTransaction != nullptr)
	{
		UE_LOG_ONLINE_PURCHASE(Warning, TEXT("Previous purchase finished without notification"));
		InProgressTransaction.Reset();
	}

	TSharedPtr<FOnlinePurchaseInProgressTransaction> RequestedTransaction = MakeShared<FOnlinePurchaseInProgressTransaction>(CheckoutRequest, UserId, EPurchaseTransactionState::NotStarted, Delegate);

	if (IsBillingClientConnected())
	{
		int32 NumOffers = CheckoutRequest.PurchaseOffers.Num();
		if (NumOffers > 0)
		{
			extern bool AndroidThunkCpp_Iap_BeginPurchase(const TArray<FStringView>&, const FString&);
			FString AccountId;
			if (CheckoutRequest.AccountId.IsSet())
			{
				AccountId = CheckoutRequest.AccountId.GetValue();
			}
			TArray<FStringView> ProductIds;
			Algo::Transform(CheckoutRequest.PurchaseOffers, ProductIds, [](const FPurchaseCheckoutRequest::FPurchaseOfferEntry& Offer) { return FStringView(Offer.OfferId);});
			bStarted = AndroidThunkCpp_Iap_BeginPurchase(ProductIds, AccountId);
			if(bStarted)
			{
				UE_LOG_ONLINE_PURCHASE(Display, TEXT("Created a transaction."));
				RequestedTransaction->PurchaseInfo.TransactionState = EPurchaseTransactionState::Processing;
			}
			else
			{
				UE_LOG_ONLINE_PURCHASE(Display, TEXT("Failed to create a transaction."));
				ErrorMessage = NSLOCTEXT("GooglePlayPurchase", "ErrorFailedToBeginPurchase", "Failed to checkout, failed to begin the purchase.");
				RequestedTransaction->PurchaseInfo.TransactionState = EPurchaseTransactionState::Failed;
			}
		}
		else
		{
			ErrorMessage = NSLOCTEXT("GooglePlayPurchase", "ErrorNoOffersSpecified", "Failed to checkout, no offers given.");
			RequestedTransaction->PurchaseInfo.TransactionState = EPurchaseTransactionState::Failed;
		}
	}
	else
	{
		ErrorMessage = NSLOCTEXT("GooglePlayPurchase", "ErrorBillingClientNotConnected", "Failed to checkout, the billing client is not connected.");
		RequestedTransaction->PurchaseInfo.TransactionState = EPurchaseTransactionState::Failed;
	}

	if(bStarted)
	{
		InProgressTransaction = MoveTemp(RequestedTransaction);
	}
	else
	{
		TSharedRef<FPurchaseReceipt> FailReceipt = RequestedTransaction->GenerateReceipt();

		Subsystem->ExecuteNextTick([ErrorMessage, FailReceipt, Delegate]()
		{
			FOnlineError Error(ErrorMessage);
			Delegate.ExecuteIfBound(Error, FailReceipt);
		});
	}
}

void FOnlinePurchaseGooglePlay::Checkout(const FUniqueNetId& UserId, const FPurchaseCheckoutRequest& CheckoutRequest, const FOnPurchaseReceiptlessCheckoutComplete& Delegate)
{
	// Not Supported
	UE_LOG_ONLINE(Error, TEXT("FOnlinePurchaseGooglePlay::Checkout (receiptless) is not currently supported"));
	Delegate.ExecuteIfBound(FOnlineError(EOnlineErrorResult::NotImplemented));
}

void FOnlinePurchaseGooglePlay::FinalizePurchase(const FUniqueNetId& UserId, const FString& ReceiptId)
{
	if (!bDisableLocalAcknowledgeAndConsume)
	{
		// Look for known receipt with given transaction id. Call overload with 3rd argument if found
		TSharedRef<FPurchaseReceipt>* MatchingReceipt = KnownTransactions.FindByPredicate([&ReceiptId](const TSharedRef<FPurchaseReceipt>& Receipt)
			{
				return ReceiptId == Receipt->TransactionId;
			});
		if (MatchingReceipt != nullptr)
		{
			FinalizePurchase(UserId, ReceiptId, (*MatchingReceipt)->ReceiptOffers[0].LineItems[0].ValidationInfo);
		}
		else
		{
			// Always acknowledge to avoid failing on subscriptions
			extern void AndroidThunkCpp_Iap_AcknowledgePurchase(const FString&);
			AndroidThunkCpp_Iap_AcknowledgePurchase(ReceiptId);

			UE_LOG_ONLINE_PURCHASE(Warning, TEXT("FOnlinePurchaseGooglePlay::Receipt not found matching transaction id %s. Will consume product to keep backwards compatibility"), *ReceiptId);
			// Keep previous behaviour: All products are considered consumable. Subscriptions cannot be consumed. An error will be logged when trying to consume them
			extern void AndroidThunkCpp_Iap_ConsumePurchase(const FString&);
			AndroidThunkCpp_Iap_ConsumePurchase(ReceiptId);
		}
	}
}

void FOnlinePurchaseGooglePlay::FinalizePurchase(const FUniqueNetId& UserId, const FString& ReceiptId, const FString& ReceiptInfo)
{
	if (!bDisableLocalAcknowledgeAndConsume)
	{
		if(ReceiptInfo.Contains(FGoogleTransactionData::SubscriptionReceiptMarker))
		{
			extern void AndroidThunkCpp_Iap_AcknowledgePurchase(const FString&);
			AndroidThunkCpp_Iap_AcknowledgePurchase(ReceiptId);
		}
		else
		{
			extern void AndroidThunkCpp_Iap_ConsumePurchase(const FString&);
			AndroidThunkCpp_Iap_ConsumePurchase(ReceiptId);
		}
	}
}

void FOnlinePurchaseGooglePlay::RedeemCode(const FUniqueNetId& UserId, const FRedeemCodeRequest& RedeemCodeRequest, const FOnPurchaseRedeemCodeComplete& Delegate)
{
	// Not supported
	FOnlineError Result;
	Delegate.ExecuteIfBound(Result, MakeShared<FPurchaseReceipt>());
}

void FOnlinePurchaseGooglePlay::QueryReceipts(const FUniqueNetId& UserId, bool /*bRestoreReceipts*/, const FOnQueryReceiptsComplete& Delegate)
{
	if (!bQueryingReceipts)
	{
		/**
		 * bRestoreReceipts is irrelevant because there is no concept of restore on GooglePlay, any purchase query will reveal all non consumed purchases. 
		 */
		bQueryingReceipts = true;
		QueryReceiptsComplete = Delegate;

		extern bool AndroidThunkCpp_Iap_QueryExistingPurchases();
		if (!AndroidThunkCpp_Iap_QueryExistingPurchases())
		{
			UE_LOG_ONLINE_PURCHASE(Verbose, TEXT("FOnlinePurchaseGooglePlay::QueryReceipts failed to start query"));
			
			Subsystem->ExecuteNextTick([this]() {
				FOnlineError Result(false);
				QueryReceiptsComplete.ExecuteIfBound(Result);
				QueryReceiptsComplete.Unbind();
				bQueryingReceipts = false;
			});
		}
	}
	else
	{
		UE_LOG_ONLINE_PURCHASE(Verbose, TEXT("FOnlinePurchaseGooglePlay::QueryReceipts already in progress."));
		
		Subsystem->ExecuteNextTick([Delegate = Delegate]() mutable {
			FOnlineError Result(false);
			Delegate.ExecuteIfBound(Result);
		});
	}
}

void FOnlinePurchaseGooglePlay::GetReceipts(const FUniqueNetId& UserId, TArray<FPurchaseReceipt>& OutReceipts) const
{
	OutReceipts.Empty();

	for(const TSharedRef<FPurchaseReceipt>& Transaction: KnownTransactions)
	{
		OutReceipts.Add(*Transaction);
	}
	
	UE_LOG_ONLINE_PURCHASE(Log, TEXT("FOnlinePurchaseGooglePlay::GetReceipts - Final Number of OutReceipts - %d"), OutReceipts.Num());
}

void FOnlinePurchaseGooglePlay::ShowAlternativeBillingInformationDialog(const FOnShowAlternativeBillingInformationDialogComplete& Delegate)
{
	UE_LOG_ONLINE_PURCHASE(Verbose, TEXT("FOnlinePurchaseGooglePlay::ShowAlternativeBillingOnlyInformationDialog"));

	if (!bRequestedAlternativeBillingOnlyInformationDialog)
	{
		bRequestedAlternativeBillingOnlyInformationDialog = true;
		AlternativeBillingOnlyInformationDialogComplete = Delegate;

		extern bool AndroidThunkJava_IapShowAlternativeBillingOnlyInformationDialog();
		if (!AndroidThunkJava_IapShowAlternativeBillingOnlyInformationDialog())
		{
			UE_LOG_ONLINE_PURCHASE(Verbose, TEXT("FOnlinePurchaseGooglePlay::ShowAlternativeBillingInformationDialog failed to start dialog"));

			Subsystem->ExecuteNextTick([this]() {
				FOnlineError Result(EOnlineErrorResult::RequestFailure);
				AlternativeBillingOnlyInformationDialogComplete.ExecuteIfBound(Result);
				AlternativeBillingOnlyInformationDialogComplete.Unbind();
				bRequestedAlternativeBillingOnlyInformationDialog = false;
			});
		}
	}
	else
	{

		UE_LOG_ONLINE_PURCHASE(Verbose, TEXT("FOnlinePurchaseGooglePlay::ShowAlternativeBillingInformationDialog already in progress."));

		Subsystem->ExecuteNextTick([Delegate = Delegate]() mutable {
			FOnlineError Result(false);
			Delegate.ExecuteIfBound(Result);
		});
	}
}

void FOnlinePurchaseGooglePlay::OnAlternativeBillingOnlyInformationDialogComplete(EGooglePlayBillingResponseCode InResponseCode)
{
	UE_LOG_ONLINE_PURCHASE(Log, TEXT("FOnlinePurchaseGooglePlay::OnAlternativeBillingOnlyInformationDialogComplete %s"), LexToString(InResponseCode));

	if (bRequestedAlternativeBillingOnlyInformationDialog)
	{
		bool bSuccess = (InResponseCode == EGooglePlayBillingResponseCode::Ok);
		FOnlineError Result(bSuccess);

		if (!bSuccess)
		{
			Result.SetFromErrorCode(LexToString(InResponseCode));
			UE_LOG_ONLINE_PURCHASE(Log, TEXT("OnAlternativeBillingOnlyInformationDialogComplete failed %s"), LexToString(InResponseCode));
		}

		AlternativeBillingOnlyInformationDialogComplete.ExecuteIfBound(Result);
		AlternativeBillingOnlyInformationDialogComplete.Unbind();
		bRequestedAlternativeBillingOnlyInformationDialog = false;
	}
	else
	{
		UE_LOG_ONLINE_PURCHASE(Warning, TEXT("FOnlinePurchaseGooglePlay::OnAlternativeBillingOnlyInformationDialogComplete unexpected call %s"), LexToString(InResponseCode));
	}
}

void FOnlinePurchaseGooglePlay::OnTransactionCompleteResponse(EGooglePlayBillingResponseCode InResponseCode, const FGoogleTransactionData& InTransactionData)
{
	UE_LOG_ONLINE_PURCHASE(Log, TEXT("FOnlinePurchaseGooglePlay::OnTransactionCompleteResponse %s. Transaction: %s"), LexToString(InResponseCode), *InTransactionData.ToDebugString());
	EPurchaseTransactionState Result = ConvertGPResponseToPurchaseTransactionState(InResponseCode, InTransactionData.GetPurchaseState());

	bool bIsInProgressTransaction = false;
	
	if(InProgressTransaction != nullptr)
	{
		bool bWasInProgressTransactionCanceled = (Result == EPurchaseTransactionState::Canceled && InTransactionData.GetOfferIds().IsEmpty());
		bool bTransactionMatchesInProgressRequest = InTransactionData.IsMatchingRequest(InProgressTransaction->CheckoutRequest); 

		bIsInProgressTransaction = bWasInProgressTransactionCanceled || bTransactionMatchesInProgressRequest;
	}

	if(bIsInProgressTransaction)
	{
		const FString& ErrorStr = InTransactionData.GetErrorStr();

		FOnlineError FinalResult;
		switch (Result)
		{
			case EPurchaseTransactionState::Failed:
				FinalResult.SetFromErrorCode(FString::Printf(TEXT("com.epicgames.purchase.googleplay.failure[%d:%s]"), EnumToUnderlyingType(InResponseCode), LexToString(InResponseCode)));
				FinalResult.ErrorMessage = !ErrorStr.IsEmpty() ? FText::FromString(ErrorStr) : LOCTEXT("GooglePlayTransactionFailed", "Transaction Failed");
				if (InResponseCode == EGooglePlayBillingResponseCode::BillingUnavailable)
				{
					FinalResult.SetErrorRaw(FString::Printf(TEXT("%s.noDelayedRedeemAndError"), *FinalResult.GetErrorCode()));
				}
				break;
			case EPurchaseTransactionState::Canceled:
				FinalResult.SetFromErrorCode(TEXT("com.epicgames.catalog_helper.user_cancelled"));
				FinalResult.ErrorMessage = !ErrorStr.IsEmpty() ? FText::FromString(ErrorStr) : LOCTEXT("GooglePlayTransactionCancel", "Transaction Canceled");
				break;
			case EPurchaseTransactionState::Purchased:
				FinalResult.bSucceeded = true;
				break;
			case EPurchaseTransactionState::Deferred:
				FinalResult.SetFromErrorCode(TEXT("com.epicgames.purchase.deferred"));
				FinalResult.ErrorMessage = !ErrorStr.IsEmpty() ? FText::FromString(ErrorStr) : LOCTEXT("GooglePlayTransactionDeferred", "Transaction Deferred");
				break;
			case EPurchaseTransactionState::Invalid:
				FinalResult.SetFromErrorCode(FString::Printf(TEXT("com.epicgames.purchase.googleplay.invalid[%d:%s]"), EnumToUnderlyingType(InResponseCode), LexToString(InResponseCode)));
				FinalResult.ErrorMessage = !ErrorStr.IsEmpty() ? FText::FromString(ErrorStr) : LOCTEXT("GooglePlayInvalidState", "Invalid purchase result");
				break;
			default:
				UE_LOG_ONLINE_PURCHASE(Warning, TEXT("Unexpected EPurchaseTransactionState %d"), (int)Result);
				FinalResult.SetFromErrorCode(TEXT("com.epicgames.purchase.unexpected_state"));
				FinalResult.ErrorMessage = !ErrorStr.IsEmpty() ? FText::FromString(ErrorStr) : LOCTEXT("GooglePlayUnexpectedState", "Unexpected purchase result");
				break;
		}

		InProgressTransaction->PurchaseInfo.TransactionState = Result;
		InProgressTransaction->PurchaseInfo.TransactionId = InTransactionData.GetTransactionIdentifier();

		if (Result == EPurchaseTransactionState::Purchased ||
			Result == EPurchaseTransactionState::Deferred )
		{
			InProgressTransaction->AddCompletedOffer(Result, InTransactionData);
		}

		TSharedRef<FPurchaseReceipt> FinalReceipt = InProgressTransaction->GenerateReceipt();

		if (Result == EPurchaseTransactionState::Purchased ||
			Result == EPurchaseTransactionState::Deferred )
		{
			KnownTransactions.Add(FinalReceipt);
		}
		InProgressTransaction->CheckoutCompleteDelegate.ExecuteIfBound(FinalResult, FinalReceipt);
		InProgressTransaction.Reset();
	}
	else
	{
		if (Result == EPurchaseTransactionState::Purchased)
		{
			UE_LOG_ONLINE_PURCHASE(Log, TEXT("Unexpected transaction completed (maybe a pending transaction finished?)"));
			TriggerOnUnexpectedPurchaseReceiptDelegates(*FUniqueNetIdGooglePlay::EmptyId());
		}
	}
}

void FOnlinePurchaseGooglePlay::OnQueryExistingPurchasesComplete(EGooglePlayBillingResponseCode InResponseCode, const TArray<FGoogleTransactionData>& InExistingPurchases)
{
	UE_LOG_ONLINE_PURCHASE(Log, TEXT("FOnlinePurchaseGooglePlay::OnQueryExistingPurchasesComplete Response: %s Num: %d"), LexToString(InResponseCode), InExistingPurchases.Num());

	if (bQueryingReceipts)
	{
		bool bSuccess = (InResponseCode == EGooglePlayBillingResponseCode::Ok);
		if (bSuccess)
		{
			KnownTransactions.Empty();

			for(const FGoogleTransactionData& Purchase: InExistingPurchases)
			{
				UE_LOG_ONLINE_PURCHASE(Log, TEXT("Adding existing receipt %s"), *Purchase.ToDebugString());
				TSharedRef<FPurchaseReceipt> Receipt = FOnlinePurchaseInProgressTransaction::GenerateReceipt(Purchase); 
				KnownTransactions.Add(Receipt);
			}			
		}
		else
		{
			UE_LOG_ONLINE_PURCHASE(Log, TEXT("OnQueryExistingPurchasesComplete failed"));
		}

		FOnlineError Result(bSuccess);
		QueryReceiptsComplete.ExecuteIfBound(Result);
		QueryReceiptsComplete.Unbind();
		bQueryingReceipts = false;
	}
	else
	{
		UE_LOG_ONLINE_PURCHASE(Warning, TEXT("FOnlinePurchaseGooglePlay::OnQueryExistingPurchasesComplete unexpected call"));
	}
}

void FOnlinePurchaseGooglePlay::FinalizeReceiptValidationInfo(const FUniqueNetId& UserId, FString& InReceiptValidationInfo, const FOnFinalizeReceiptValidationInfoComplete& Delegate)
{
	FOnlineError DefaultSuccess(true);
	Delegate.ExecuteIfBound(DefaultSuccess, InReceiptValidationInfo);
}

void JNICALL UE::Jni::FGooglePlayStoreHelper::NativeQueryExistingPurchasesComplete(JNIEnv* jenv, jobject /*Thiz*/, jint ResponseCode, Java::Lang::TArray<jstring, 2>* ProductIDs, Java::Lang::TArray<jint>* PurchaseStatesArray, Java::Lang::TArray<jstring>* PurchaseTokens, Java::Lang::TArray<jstring>* ReceiptsData, Java::Lang::TArray<jstring>* Signatures)
{
	TArray<FGoogleTransactionData> ExistingPurchaseInfo;

	EGooglePlayBillingResponseCode EGPResponse = (EGooglePlayBillingResponseCode)ResponseCode;

	bool bWasSuccessful = (EGPResponse == EGooglePlayBillingResponseCode::Ok);
	UE_LOG_ONLINE_PURCHASE(Verbose, TEXT("NativeQueryExistingPurchasesComplete received response code: %s\n"), LexToString(EGPResponse));

	if (bWasSuccessful)
	{
		jsize NumProducts = jenv->GetArrayLength(ProductIDs);
		jsize NumTransactionStates = jenv->GetArrayLength(PurchaseStatesArray);
		jsize NumPurchaseTokens = jenv->GetArrayLength(PurchaseTokens);
		jsize NumReceipts = jenv->GetArrayLength(ReceiptsData);
		jsize NumSignatures = jenv->GetArrayLength(Signatures);

		ensure((NumProducts == NumPurchaseTokens) && (NumProducts == NumTransactionStates) && (NumProducts == NumReceipts) && (NumProducts == NumSignatures));

		jint* PurchaseStates = jenv->GetIntArrayElements(PurchaseStatesArray, NULL);
		
		for (jsize Idx = 0; Idx < NumProducts; Idx++)
		{
			// Build the product information strings.
			const auto OfferIds = FJavaHelper::ObjectArrayToFStringTArray(jenv, (jobjectArray)jenv->GetObjectArrayElement(ProductIDs, Idx));
			const auto PurchaseToken = FJavaHelper::FStringFromLocalRef(jenv, (jstring)jenv->GetObjectArrayElement(PurchaseTokens, Idx));
			const auto ReceiptData = FJavaHelper::FStringFromLocalRef(jenv, (jstring)jenv->GetObjectArrayElement(ReceiptsData, Idx));
			const auto SignatureData = FJavaHelper::FStringFromLocalRef(jenv, (jstring)jenv->GetObjectArrayElement(Signatures, Idx));
			const auto PurchaseState = (EGooglePlayPurchaseState)PurchaseStates[Idx];

			FGoogleTransactionData ExistingPurchase(OfferIds, PurchaseToken, ReceiptData, SignatureData, PurchaseState);
			ExistingPurchaseInfo.Add(ExistingPurchase);

			UE_LOG_ONLINE_PURCHASE(Verbose, TEXT("Existing Product Identifier: %s"), *ExistingPurchase.ToDebugString());
		}
		jenv->ReleaseIntArrayElements(PurchaseStatesArray, PurchaseStates, JNI_ABORT);
	}
	
	if (auto OnlineSubGP = static_cast<FOnlineSubsystemGooglePlay* const>(IOnlineSubsystem::Get(GOOGLEPLAY_SUBSYSTEM)))
	{
		OnlineSubGP->ExecuteNextTick([OnlineSubGP, EGPResponse, ExistingPurchaseInfo = MoveTemp(ExistingPurchaseInfo)]()
		{
			TSharedPtr<FOnlinePurchaseGooglePlay> PurchaseInt = StaticCastSharedPtr<FOnlinePurchaseGooglePlay>(OnlineSubGP->GetPurchaseInterface());
			PurchaseInt->OnQueryExistingPurchasesComplete(EGPResponse, ExistingPurchaseInfo);
		});
	}
}

void JNICALL UE::Jni::FGooglePlayStoreHelper::NativePurchaseComplete(JNIEnv* jenv, jobject /*Thiz*/, jint JavaResponseCode, Java::Lang::TArray<jstring>* JavaProductIds, jint JavaPurchaseState, jstring JavaPurchaseToken, jstring JavaReceiptData, jstring JavaSignature)
{
	EGooglePlayBillingResponseCode EGPResponse = (EGooglePlayBillingResponseCode)JavaResponseCode;
	EGooglePlayPurchaseState PurchaseState = (EGooglePlayPurchaseState)JavaPurchaseState;

	TArray<FString> ProductIds = FJavaHelper::ObjectArrayToFStringTArray(jenv, JavaProductIds);
	FString PurchaseToken = FJavaHelper::FStringFromParam(jenv, JavaPurchaseToken);
	FString ReceiptData = FJavaHelper::FStringFromParam(jenv, JavaReceiptData);
	FString Signature = FJavaHelper::FStringFromParam(jenv, JavaSignature);

	FGoogleTransactionData TransactionData(ProductIds, PurchaseToken, ReceiptData, Signature, PurchaseState);

	UE_LOG_ONLINE_PURCHASE(Verbose, TEXT("NativePurchaseComplete received response code: %s. Transaction: %s\n"), LexToString(EGPResponse), *TransactionData.ToDebugString());

	if (auto OnlineSubGP = static_cast<FOnlineSubsystemGooglePlay* const>(IOnlineSubsystem::Get(GOOGLEPLAY_SUBSYSTEM)))
	{
		OnlineSubGP->ExecuteNextTick([OnlineSubGP, EGPResponse, TransactionData = MoveTemp(TransactionData)]()
		{
			TSharedPtr<FOnlinePurchaseGooglePlay> PurchaseInt = StaticCastSharedPtr<FOnlinePurchaseGooglePlay>(OnlineSubGP->GetPurchaseInterface());
			PurchaseInt->OnTransactionCompleteResponse(EGPResponse, TransactionData);
		});
	}
}

void JNICALL UE::Jni::FGooglePlayStoreHelper::NativeConsumeComplete(JNIEnv* jenv, jobject /*Thiz*/, jint JavaResponseCode, jstring JavaPurchaseToken)
{
	FString PurchaseToken = FJavaHelper::FStringFromParam(jenv, JavaPurchaseToken);;
	EGooglePlayBillingResponseCode EGPResponse = (EGooglePlayBillingResponseCode)JavaResponseCode;
	UE_LOG_ONLINE_PURCHASE(Verbose, TEXT("NativeConsumeComplete received response code: %s. PurchaseToken: %s\n"), LexToString(EGPResponse), *PurchaseToken);
}

void JNICALL UE::Jni::FGooglePlayStoreHelper::NativeAcknowledgeComplete(JNIEnv* jenv, jobject /*Thiz*/, jint JavaResponseCode, jstring JavaPurchaseToken)
{
	FString PurchaseToken = FJavaHelper::FStringFromParam(jenv, JavaPurchaseToken);;
	EGooglePlayBillingResponseCode EGPResponse = (EGooglePlayBillingResponseCode)JavaResponseCode;
	UE_LOG_ONLINE_PURCHASE(Verbose, TEXT("NativeAcknowledgeComplete received response code: %s. PurchaseToken: %s\n"), LexToString(EGPResponse), *PurchaseToken);
}

void JNICALL UE::Jni::FGooglePlayStoreHelper::NativeAlternativeBillingOnlyInformationDialogComplete(JNIEnv* jenv, jobject /*Thiz*/, jint JavaResponseCode)
{
	EGooglePlayBillingResponseCode EGPResponse = (EGooglePlayBillingResponseCode)JavaResponseCode;

	UE_LOG_ONLINE_PURCHASE(Verbose, TEXT("NativeAlternativeBillingOnlyInformationDialogComplete received response code: %s\n"), LexToString(EGPResponse));

	if (auto OnlineSubGP = static_cast<FOnlineSubsystemGooglePlay* const>(IOnlineSubsystem::Get(GOOGLEPLAY_SUBSYSTEM)))
	{
		OnlineSubGP->ExecuteNextTick([OnlineSubGP, EGPResponse]()
		{
			TSharedPtr<FOnlinePurchaseGooglePlay> PurchaseInt = StaticCastSharedPtr<FOnlinePurchaseGooglePlay>(OnlineSubGP->GetPurchaseInterface());
			PurchaseInt->OnAlternativeBillingOnlyInformationDialogComplete(EGPResponse);
		});
	}
}

#undef LOCTEXT_NAMESPACE

