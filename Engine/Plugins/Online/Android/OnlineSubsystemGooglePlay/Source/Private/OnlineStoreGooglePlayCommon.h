// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/OnlinePurchaseInterface.h"
#include "Interfaces/OnlineStoreInterfaceV2.h"

struct FGoogleTransactionData;

/** Possible responses returned from the Java GooglePlay billing interface.
 *  CustomLogicError is a custom Unreal Engine error to represent logic errors 
 *  on the Java side
 */
enum class EGooglePlayBillingResponseCode : int8
{
	CustomLogicError = -127, // Custom value. Should match with GooglePlayStoreHelper.CustomLogicErrorResponse on GooglePlayStoreHelper.java
	FeatureNotSupported = -2,
	ServiceDisconnected = -1,
	Ok = 0,
	UserCancelled = 1,
	ServiceUnavailable = 2,
	BillingUnavailable = 3,
	ItemUnavailable = 4,
	DeveloperError = 5,
	Error = 6,
	ItemAlreadyOwned = 7,
	ItemNotOwned = 8,
	NetworkError = 12,
};

/* Possible transaction states returned from the Java GooglePlay billing interface.*/
enum class EGooglePlayPurchaseState : uint8
{
	UnspecifiedState = 0,
	Purchased = 1,
	Pending = 2,
};

inline const TCHAR* const LexToString(EGooglePlayBillingResponseCode InResponseCode)
{
	switch (InResponseCode)
	{
		case EGooglePlayBillingResponseCode::CustomLogicError:
			return TEXT("CustomLogicError");
		case EGooglePlayBillingResponseCode::FeatureNotSupported:
			return TEXT("FeatureNotSupported");
		case EGooglePlayBillingResponseCode::ServiceDisconnected:
			return TEXT("ServiceDisconnected");
		case EGooglePlayBillingResponseCode::Ok:
			return TEXT("Ok");
		case EGooglePlayBillingResponseCode::UserCancelled:
			return TEXT("UserCancelled");
		case EGooglePlayBillingResponseCode::ItemAlreadyOwned:
			return TEXT("ItemAlreadyOwned");
		case EGooglePlayBillingResponseCode::ItemNotOwned:
			return TEXT("ItemNotOwned");
		case EGooglePlayBillingResponseCode::ServiceUnavailable:
			return TEXT("ServiceUnavailable");
		case EGooglePlayBillingResponseCode::BillingUnavailable:
			return TEXT("BillingUnavailable");
		case EGooglePlayBillingResponseCode::ItemUnavailable:
			return TEXT("ItemUnavailable");
		case EGooglePlayBillingResponseCode::DeveloperError:
			return TEXT("DeveloperError");
		case EGooglePlayBillingResponseCode::Error:
			return TEXT("Error");
		case EGooglePlayBillingResponseCode::NetworkError:
			return TEXT("NetworkError");
		default:
			return TEXT("UnknownError");
	}
}

inline const TCHAR* const LexToString(EGooglePlayPurchaseState InTransactionState)
{
	switch (InTransactionState)
	{
		case EGooglePlayPurchaseState::UnspecifiedState:
			return TEXT("UnspecifiedState");
		case EGooglePlayPurchaseState::Purchased:
			return TEXT("Purchased");
		case EGooglePlayPurchaseState::Pending:
			return TEXT("Pending");
		default:
			return TEXT("UnknownState");
	}
}

inline EInAppPurchaseState::Type ConvertGPResponseCodeToIAPState(const EGooglePlayBillingResponseCode InResponseCode)
{
	switch (InResponseCode)
	{
		case EGooglePlayBillingResponseCode::Ok:
			return EInAppPurchaseState::Success;
		case EGooglePlayBillingResponseCode::UserCancelled:
			return EInAppPurchaseState::Cancelled;
		case EGooglePlayBillingResponseCode::ItemAlreadyOwned:
			return EInAppPurchaseState::AlreadyOwned;
		case EGooglePlayBillingResponseCode::ItemNotOwned:
			return EInAppPurchaseState::NotAllowed;
		case EGooglePlayBillingResponseCode::CustomLogicError:
		case EGooglePlayBillingResponseCode::FeatureNotSupported:
		case EGooglePlayBillingResponseCode::ServiceDisconnected:
		case EGooglePlayBillingResponseCode::ServiceUnavailable:
		case EGooglePlayBillingResponseCode::BillingUnavailable:
		case EGooglePlayBillingResponseCode::ItemUnavailable:
		case EGooglePlayBillingResponseCode::DeveloperError:
		case EGooglePlayBillingResponseCode::Error:
		default:
			return EInAppPurchaseState::Failed;
	}
}

inline EPurchaseTransactionState ConvertGPPurchaseStateToPurchaseTransactionState(const EGooglePlayPurchaseState InTransactionState)
{
	switch(InTransactionState)
	{
		case EGooglePlayPurchaseState::Purchased:
			return EPurchaseTransactionState::Purchased;
		case EGooglePlayPurchaseState::Pending:
			return EPurchaseTransactionState::Deferred;
		default:
			return EPurchaseTransactionState::Invalid;
	}
}

inline EPurchaseTransactionState ConvertGPResponseToPurchaseTransactionState(const EGooglePlayBillingResponseCode InResponseCode, const EGooglePlayPurchaseState InPurchaseState)
{
	switch (InResponseCode)
	{
		case EGooglePlayBillingResponseCode::Ok:
			return ConvertGPPurchaseStateToPurchaseTransactionState(InPurchaseState);
		case EGooglePlayBillingResponseCode::UserCancelled:
			return EPurchaseTransactionState::Canceled;
		case EGooglePlayBillingResponseCode::ItemAlreadyOwned:
			// Non consumable purchased again or consumable purchased again before consuming
			return EPurchaseTransactionState::Invalid;
		case EGooglePlayBillingResponseCode::ItemNotOwned:
			return EPurchaseTransactionState::Invalid;
		case EGooglePlayBillingResponseCode::CustomLogicError:
		case EGooglePlayBillingResponseCode::FeatureNotSupported:
		case EGooglePlayBillingResponseCode::ServiceDisconnected:
		case EGooglePlayBillingResponseCode::ServiceUnavailable:
		case EGooglePlayBillingResponseCode::BillingUnavailable:
		case EGooglePlayBillingResponseCode::ItemUnavailable:
		case EGooglePlayBillingResponseCode::DeveloperError:
		case EGooglePlayBillingResponseCode::Error:
		default:
			return EPurchaseTransactionState::Failed;
	}
}

typedef FOnlineStoreOffer FProvidedProductInformation;

 /**
  * Delegate fired when an IAP query for available offers has completed
  *
  * @param Response response from GooglePlay backend
  * @param ProvidedProductInformation list of offers returned in response to a query on available offer ids
  */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnGooglePlayAvailableIAPQueryComplete, EGooglePlayBillingResponseCode /*Response*/, const TArray<FProvidedProductInformation>& /*ProvidedProductInformation*/);
typedef FOnGooglePlayAvailableIAPQueryComplete::FDelegate FOnGooglePlayAvailableIAPQueryCompleteDelegate;

/**
 * Delegate fired when an IAP has completed
 *
 * @param InResponseCode response from the GooglePlay backend
 * @param InTransactionData transaction data for the completed purchase
 */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnGooglePlayProcessPurchaseComplete, EGooglePlayBillingResponseCode /*InResponseCode*/, const FGoogleTransactionData& /*InTransactionData*/);
typedef FOnGooglePlayProcessPurchaseComplete::FDelegate FOnGooglePlayProcessPurchaseCompleteDelegate;

/**
 * Delegate fired internally when an existing purchases query has completed
 *
 * @param InResponseCode response from the GooglePlay backend
 * @param InExistingPurchases known purchases for the user (non consumed or permanent)
 */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnGooglePlayQueryExistingPurchasesComplete, EGooglePlayBillingResponseCode /*InResponseCode*/, const TArray<FGoogleTransactionData>& /*InExistingPurchases*/);
typedef FOnGooglePlayQueryExistingPurchasesComplete::FDelegate FOnGooglePlayQueryExistingPurchasesCompleteDelegate;

/**
 * Delegate fired internally when a consume purchase query has completed
 *
 * @param InResponseCode response from the GooglePlay backend
 * @param InPurchaseToken purchase token for consumed purchase
 */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnGooglePlayConsumePurchaseComplete, EGooglePlayBillingResponseCode /*InResponseCode*/, const FString& /*InPurchaseToken*/);
typedef FOnGooglePlayConsumePurchaseComplete::FDelegate FOnGooglePlayConsumePurchaseCompleteDelegate;
