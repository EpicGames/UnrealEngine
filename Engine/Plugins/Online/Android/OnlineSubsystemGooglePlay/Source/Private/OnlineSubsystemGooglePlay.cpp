// Copyright Epic Games, Inc. All Rights Reserved.

//Google Play Services

#include "OnlineSubsystemGooglePlay.h"
#include "OnlineIdentityErrors.h"
#include "OnlineIdentityInterfaceGooglePlay.h"
#include "OnlineStoreGooglePlayCommon.h"
#include "OnlineAchievementsInterfaceGooglePlay.h"
#include "OnlineLeaderboardInterfaceGooglePlay.h"
#include "OnlineExternalUIInterfaceGooglePlay.h"
#include "OnlinePurchaseGooglePlay.h"
#include "OnlineStoreGooglePlay.h"
#include "OnlineAsyncTaskManagerGooglePlay.h"
#include "Misc/ConfigCacheIni.h"
#include "Async/TaskGraphInterfaces.h"
#include "Stats/Stats.h"

FOnlineSubsystemGooglePlay::FOnlineSubsystemGooglePlay(FName InInstanceName)
	: FOnlineSubsystemImpl(GOOGLEPLAY_SUBSYSTEM, InInstanceName)
	, IdentityInterface(nullptr)
	, LeaderboardsInterface(nullptr)
	, AchievementsInterface(nullptr)
{
}

IOnlineIdentityPtr FOnlineSubsystemGooglePlay::GetIdentityInterface() const
{
	return IdentityInterface;
}

IOnlineStoreV2Ptr FOnlineSubsystemGooglePlay::GetStoreV2Interface() const
{
	return StoreV2Interface;
}

IOnlinePurchasePtr FOnlineSubsystemGooglePlay::GetPurchaseInterface() const
{
	return PurchaseInterface;
}

IOnlineSessionPtr FOnlineSubsystemGooglePlay::GetSessionInterface() const
{
	return nullptr;
}

IOnlineFriendsPtr FOnlineSubsystemGooglePlay::GetFriendsInterface() const
{
	return nullptr;
}

IOnlinePartyPtr FOnlineSubsystemGooglePlay::GetPartyInterface() const
{
	return nullptr;
}

IOnlineGroupsPtr FOnlineSubsystemGooglePlay::GetGroupsInterface() const
{
	return nullptr;
}

IOnlineSharedCloudPtr FOnlineSubsystemGooglePlay::GetSharedCloudInterface() const
{
	return nullptr;
}

IOnlineUserCloudPtr FOnlineSubsystemGooglePlay::GetUserCloudInterface() const
{
	return nullptr;
}

IOnlineLeaderboardsPtr FOnlineSubsystemGooglePlay::GetLeaderboardsInterface() const
{
	return LeaderboardsInterface;
}

IOnlineVoicePtr FOnlineSubsystemGooglePlay::GetVoiceInterface() const
{
	return nullptr;
}

IOnlineExternalUIPtr FOnlineSubsystemGooglePlay::GetExternalUIInterface() const
{
	return ExternalUIInterface;
}

IOnlineTimePtr FOnlineSubsystemGooglePlay::GetTimeInterface() const
{
	return nullptr;
}

IOnlineTitleFilePtr FOnlineSubsystemGooglePlay::GetTitleFileInterface() const
{
	return nullptr;
}

IOnlineEntitlementsPtr FOnlineSubsystemGooglePlay::GetEntitlementsInterface() const
{
	return nullptr;
}

IOnlineAchievementsPtr FOnlineSubsystemGooglePlay::GetAchievementsInterface() const
{
	return AchievementsInterface;
}

bool FOnlineSubsystemGooglePlay::Init() 
{
	FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FOnlineSubsystemGooglePlay::Init"));
	
	OnlineAsyncTaskThreadRunnable.Reset(new FOnlineAsyncTaskManagerGooglePlay);
	OnlineAsyncTaskThread.Reset(FRunnableThread::Create(OnlineAsyncTaskThreadRunnable.Get(), *FString::Printf(TEXT("OnlineAsyncTaskThread %s"), *InstanceName.ToString())));

	GooglePlayGamesWrapper.Init();

	IdentityInterface = MakeShareable(new FOnlineIdentityGooglePlay(this));
	LeaderboardsInterface = MakeShareable(new FOnlineLeaderboardsGooglePlay(this));
	AchievementsInterface = MakeShareable(new FOnlineAchievementsGooglePlay(this));
	ExternalUIInterface = MakeShareable(new FOnlineExternalUIGooglePlay(this));

	if (IsInAppPurchasingEnabled())
	{
        StoreV2Interface = MakeShareable(new FOnlineStoreGooglePlayV2(this));
		PurchaseInterface = MakeShareable(new FOnlinePurchaseGooglePlay(this));
		PurchaseInterface->Init();
	}
	
	return true;
}

bool FOnlineSubsystemGooglePlay::Tick(float DeltaTime)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSubsystemGooglePlay_Tick);

	if (!FOnlineSubsystemImpl::Tick(DeltaTime))
	{
		return false;
	}

	if (OnlineAsyncTaskThreadRunnable)
	{
		OnlineAsyncTaskThreadRunnable->GameTick();
	}

	return true;
}


bool FOnlineSubsystemGooglePlay::Shutdown() 
{
	UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineSubsystemGooglePlay::Shutdown()"));

	FOnlineSubsystemImpl::Shutdown();

#define DESTRUCT_INTERFACE(Interface) \
	if (Interface.IsValid()) \
	{ \
		ensure(Interface.IsUnique()); \
		Interface = NULL; \
	}

	// Destruct the interfaces
	DESTRUCT_INTERFACE(StoreV2Interface);
	DESTRUCT_INTERFACE(PurchaseInterface);
	DESTRUCT_INTERFACE(ExternalUIInterface);
	DESTRUCT_INTERFACE(AchievementsInterface);
	DESTRUCT_INTERFACE(LeaderboardsInterface);
	DESTRUCT_INTERFACE(IdentityInterface);
#undef DESTRUCT_INTERFACE

	GooglePlayGamesWrapper.Reset();

	OnlineAsyncTaskThread.Reset();
	OnlineAsyncTaskThreadRunnable.Reset();

	return true;
}


FString FOnlineSubsystemGooglePlay::GetAppId() const 
{
	//get app id from settings. 
	return TEXT( "AndroidAppIDPlaceHolder" );
}


bool FOnlineSubsystemGooglePlay::Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) 
{
	if (FOnlineSubsystemImpl::Exec(InWorld, Cmd, Ar))
	{
		return true;
	}
	return false;
}

FText FOnlineSubsystemGooglePlay::GetOnlineServiceName() const
{
	return NSLOCTEXT("OnlineSubsystemGooglePlay", "OnlineServiceName", "Google Play");
}

bool FOnlineSubsystemGooglePlay::IsEnabled() const
{
	bool bEnabled = false;

	// AndroidRuntimeSettings holds a value for editor ease of use
	if (!GConfig->GetBool(TEXT("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings"), TEXT("bEnableGooglePlaySupport"), bEnabled, GEngineIni))
	{
		UE_LOG_ONLINE(Warning, TEXT("The [/Script/AndroidRuntimeSettings.AndroidRuntimeSettings]:bEnableGooglePlaySupport flag has not been set"));

		// Fallback to regular OSS location
		bEnabled = FOnlineSubsystemImpl::IsEnabled();
	}
	return bEnabled;
}

bool FOnlineSubsystemGooglePlay::IsInAppPurchasingEnabled()
{
	bool bSupportsInAppPurchasing = false;
	GConfig->GetBool(TEXT("OnlineSubsystemGooglePlay.Store"), TEXT("bSupportsInAppPurchasing"), bSupportsInAppPurchasing, GEngineIni);

	FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FOnlineSubsystemGooglePlay::IsInAppPurchasingEnabled %d"), bSupportsInAppPurchasing);
	return bSupportsInAppPurchasing;
}

void FOnlineSubsystemGooglePlay::QueueAsyncTask(FOnlineAsyncTask* AsyncTask)
{
	check(OnlineAsyncTaskThreadRunnable);
	OnlineAsyncTaskThreadRunnable->AddToInQueue(AsyncTask);
}