// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineSubsystemGooglePlayModule.h"
#include "OnlineSubsystemGooglePlay.h"

IMPLEMENT_MODULE( FOnlineSubsystemGooglePlayModule, OnlineSubsystemGooglePlay );

/**
 * Class responsible for creating instance(s) of the subsystem
 */
class FOnlineFactoryGooglePlay : public IOnlineFactory
{

private:

	/** Single instantiation of the IOS interface */
	static FOnlineSubsystemGooglePlayPtr GooglePlaySingleton;

	virtual void DestroySubsystem()
	{
		if (GooglePlaySingleton.IsValid())
		{
			GooglePlaySingleton->Shutdown();
			GooglePlaySingleton = NULL;
		}
	}

public:

	FOnlineFactoryGooglePlay() {}
	virtual ~FOnlineFactoryGooglePlay() 
	{
		DestroySubsystem();
	}

	virtual IOnlineSubsystemPtr CreateSubsystem(FName InstanceName)
	{
		if (!GooglePlaySingleton.IsValid())
		{
			GooglePlaySingleton = MakeShared<FOnlineSubsystemGooglePlay, ESPMode::ThreadSafe>(InstanceName);
			if (GooglePlaySingleton->IsEnabled())
			{
				if(!GooglePlaySingleton->Init())
				{
					UE_LOG_ONLINE(Warning, TEXT("FOnlineSubsystemGooglePlayModule failed to initialize!"));
					DestroySubsystem();
				}
			}
			else
			{
				UE_LOG_ONLINE(Warning, TEXT("FOnlineSubsystemGooglePlayModule was disabled"));
				DestroySubsystem();
			}

			return GooglePlaySingleton;
		}

		UE_LOG_ONLINE(Warning, TEXT("Can't create more than one instance of a Google Play online subsystem!"));
		return NULL;
	}
};

FOnlineSubsystemGooglePlayPtr FOnlineFactoryGooglePlay::GooglePlaySingleton = NULL;


void FOnlineSubsystemGooglePlayModule::StartupModule()
{
	UE_LOG_ONLINE(Display, TEXT("OnlineSubsystemGooglePlayModule::StartupModule()"));

	GooglePlayFactory = new FOnlineFactoryGooglePlay();

	FOnlineSubsystemModule& OSS = FModuleManager::GetModuleChecked<FOnlineSubsystemModule>("OnlineSubsystem");
	OSS.RegisterPlatformService(GOOGLEPLAY_SUBSYSTEM, GooglePlayFactory);
}


void FOnlineSubsystemGooglePlayModule::ShutdownModule()
{
	UE_LOG_ONLINE(Display, TEXT("FOnlineSubsystemGooglePlayModule::ShutdownModule()"));

	delete GooglePlayFactory;
	GooglePlayFactory = nullptr;
}
