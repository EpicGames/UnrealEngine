// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/OnlinePurchaseInterface.h"
#include "Misc/Optional.h"
#include "Serialization/JsonSerializerMacros.h"
#include "OnlineIdentityInterfaceGooglePlay.h"

enum class EGooglePlayBillingResponseCode : int8;
enum class EGooglePlayPurchaseState : uint8;

/**
 * Holds in a common format the data that comes out of a Google purchase transaction
 */
struct FGoogleTransactionData
{
	FGoogleTransactionData(const TArray<FString>& InOfferIds, const FString& InPurchaseToken, const FString& InReceiptData, const FString& InSignature, EGooglePlayPurchaseState InPurchaseState);

	/** @return a string that prints useful debug information about this transaction */
	FString ToDebugString() const;
	/** @return offer id for this transaction */
	const FString& GetOfferId() const { return OfferIds[0]; }
	/** @return all offer ids for this transaction */
	const TArray<FString>& GetOfferIds() const { return OfferIds; }
	/** @return receipt data for this transaction */
	FString GetCombinedReceiptData() const { return CombinedTransactionData.ToJson(); }
	/** @return receipt data for this transaction */
	const FString& GetReceiptData() const { return CombinedTransactionData.ReceiptData; }
	/** @return signature for this transaction */
	const FString& GetSignature() const { return CombinedTransactionData.Signature; }
	/** @return error string for this transaction, if applicable */
	const FString& GetErrorStr() const { return ErrorStr; }
	/** @return the purchase transaction id */
	const FString& GetTransactionIdentifier() const	{ return PurchaseToken; }
	/** @return the purchase state */
	EGooglePlayPurchaseState GetPurchaseState() const { return PurchaseState; }
	/** Checks if all items reported in the transaction are present in the request */
	bool IsMatchingRequest(const FPurchaseCheckoutRequest& Request) const;
	/** marker found in the receipt data to indicate it refers to a subscription */
	inline static const TCHAR* SubscriptionReceiptMarker = TEXT("isSubscription");
private:

	/** Easy access to transmission of data required for backend validation */
	class FJsonReceiptData :
		public FJsonSerializable
	{
	public:

		FJsonReceiptData() {}
		FJsonReceiptData(const TArray<FString>& InOfferIds, const FString& InReceiptData, const FString& InSignature);

		/** Opaque store receipt data */
		TOptional<bool> IsSubscription;
		/** Opaque store receipt data */
		FString ReceiptData;
		/** Signature associated with the transaction */
		FString Signature;

		// FJsonSerializable
		BEGIN_JSON_SERIALIZER
			JSON_SERIALIZE_OPTIONAL("isSubscription", IsSubscription);
			JSON_SERIALIZE("receiptData", ReceiptData);
			JSON_SERIALIZE("signature", Signature);
		END_JSON_SERIALIZER
	};

	/** GooglePlay offer id */
	TArray<FString> OfferIds;
	/** PurchaseToken for the transaction */
	FString PurchaseToken;
	/** Error on the transaction, if applicable */
	FString ErrorStr;
	/** Combined receipt with signature in JSON */
	FJsonReceiptData CombinedTransactionData;
	/** Reported GooglePlay transaction state */
	EGooglePlayPurchaseState PurchaseState;
};

/**
 * Implementation for online purchase via GooglePlay services
 */
class FOnlinePurchaseGooglePlay :
	public IOnlinePurchase,
	public TSharedFromThis<FOnlinePurchaseGooglePlay, ESPMode::ThreadSafe>
{

public:

	// IOnlinePurchase

	virtual bool IsAllowedToPurchase(const FUniqueNetId& UserId) override;
	virtual bool IsAllowedToPurchase(const FUniqueNetId& UserId, FOnlineError& Error) override;
	virtual void Checkout(const FUniqueNetId& UserId, const FPurchaseCheckoutRequest& CheckoutRequest, const FOnPurchaseCheckoutComplete& Delegate) override;
	virtual void Checkout(const FUniqueNetId& UserId, const FPurchaseCheckoutRequest& CheckoutRequest, const FOnPurchaseReceiptlessCheckoutComplete& Delegate) override;
	virtual void FinalizePurchase(const FUniqueNetId& UserId, const FString& ReceiptId) override;
	virtual void FinalizePurchase(const FUniqueNetId& UserId, const FString& ReceiptId, const FString& ReceiptInfo) override;
	virtual void RedeemCode(const FUniqueNetId& UserId, const FRedeemCodeRequest& RedeemCodeRequest, const FOnPurchaseRedeemCodeComplete& Delegate) override;
	virtual void QueryReceipts(const FUniqueNetId& UserId, bool bRestoreReceipts, const FOnQueryReceiptsComplete& Delegate) override;
	virtual void GetReceipts(const FUniqueNetId& UserId, TArray<FPurchaseReceipt>& OutReceipts) const override;
	virtual void FinalizeReceiptValidationInfo(const FUniqueNetId& UserId, FString& InReceiptValidationInfo, const FOnFinalizeReceiptValidationInfoComplete& Delegate) override;
	
	// FOnlinePurchaseGooglePlay

	/**
	 * Constructor
	 *
	 * @param InSubsystem GooglePlay subsystem being used
	 */
	FOnlinePurchaseGooglePlay(class FOnlineSubsystemGooglePlay* InSubsystem);

	/**
	 * Destructor
	 */
	virtual ~FOnlinePurchaseGooglePlay();
	
	/** Initialize the interface */
	void Init();

	/** Show the required Google Play system alternative billing informational dialog */
	virtual void ShowAlternativeBillingInformationDialog(const FOnShowAlternativeBillingInformationDialogComplete& Delegate) override;

	/** Handle response from showing the Google Play alternative billing only information dialog **/
	void OnAlternativeBillingOnlyInformationDialogComplete(EGooglePlayBillingResponseCode InResponseCode);

	/** Handle Java side transaction completed notification */
	void OnTransactionCompleteResponse(EGooglePlayBillingResponseCode InResponseCode, const FGoogleTransactionData& InTransactionData);

	/** Handle Java side query purchases completed notification */
	void OnQueryExistingPurchasesComplete(EGooglePlayBillingResponseCode InResponseCode, const TArray<FGoogleTransactionData>& InExistingPurchases);

	static bool IsSubscriptionProductId(const FString& ProductId);
private:

	/** Returns true if the Billing Client is connected and ready */
	bool IsBillingClientConnected();
	
	/**
	 * Info used to cache and track orders in progress.
	 */
	class FOnlinePurchaseInProgressTransaction;

	/** Complete transactions */
	using FOnlinePurchasePurchasedTransactions = TArray< TSharedRef<FPurchaseReceipt> >;
	
private:
	
	/** 
	 * Acknoledge and consume are not invoked when calling FinishPurchase 
	 * If this is set to 'true' acknowledge and consume should be invoked using server to server calls
	 * after validation to avoid refunds
	*/
	bool bDisableLocalAcknowledgeAndConsume = false;

	/** Are receipts being queried */
	bool bQueryingReceipts = false;
	
	/** Transient delegate to fire when query receipts has completed */
	FOnQueryReceiptsComplete QueryReceiptsComplete;

	/** Is the system dialog already being shown */
	bool bRequestedAlternativeBillingOnlyInformationDialog = false;

	/** Delegate to call when the system dialog is finished */
	FOnShowAlternativeBillingInformationDialogComplete AlternativeBillingOnlyInformationDialogComplete;

	/** Keeps track of pending user transactions */
	TSharedPtr<FOnlinePurchaseInProgressTransaction> InProgressTransaction;
	
	/** Cache of known transactions */
	FOnlinePurchasePurchasedTransactions KnownTransactions;
	
	/** Reference to the parent subsystem */
	FOnlineSubsystemGooglePlay* Subsystem = nullptr;
};

typedef TSharedPtr<FOnlinePurchaseGooglePlay, ESPMode::ThreadSafe> FOnlinePurchaseGooglePlayPtr;
