// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/OnlineStoreInterfaceV2.h"
#include "OnlineStoreGooglePlayCommon.h"

class FOnlineSubsystemGooglePlay;
class FOnlineAsyncTaskGooglePlayQueryInAppPurchasesV2;

/**
 * Implementation for online store via GooglePlay services
 */
class FOnlineStoreGooglePlayV2 :
	public IOnlineStoreV2,
	public TSharedFromThis<FOnlineStoreGooglePlayV2, ESPMode::ThreadSafe>
{
public:

	// IOnlineStoreV2

	virtual void QueryCategories(const FUniqueNetId& UserId, const FOnQueryOnlineStoreCategoriesComplete& Delegate) override;
	virtual void GetCategories(TArray<FOnlineStoreCategory>& OutCategories) const override;
	virtual void QueryOffersByFilter(const FUniqueNetId& UserId, const FOnlineStoreFilter& Filter, const FOnQueryOnlineStoreOffersComplete& Delegate) override;
	virtual void QueryOffersById(const FUniqueNetId& UserId, const TArray<FUniqueOfferId>& OfferIds, const FOnQueryOnlineStoreOffersComplete& Delegate) override;
	virtual void GetOffers(TArray<FOnlineStoreOfferRef>& OutOffers) const override;
	virtual TSharedPtr<FOnlineStoreOffer> GetOffer(const FUniqueOfferId& OfferId) const override;

	// FOnlineStoreGooglePlayV2

	/**
	 * Constructor
	 *
	 * @param InSubsystem Online subsystem being used
	 */
	FOnlineStoreGooglePlayV2(FOnlineSubsystemGooglePlay* InSubsystem);

	/**
	 * Constructor
	 */
	FOnlineStoreGooglePlayV2();

	/**
	 * Destructor
	 */
	virtual ~FOnlineStoreGooglePlayV2();

	/** Handle Java side query available iap completed notification */
	void OnGooglePlayAvailableIAPQueryComplete(EGooglePlayBillingResponseCode InResponse, const TArray<FProvidedProductInformation>& InProvidedProductInformation);

private:
	
	void AddOffer(const TSharedRef<FOnlineStoreOffer>& NewOffer);

private:
	
	/** Mapping from offer id to product information */
	typedef TMap<FUniqueOfferId, TSharedRef<FOnlineStoreOffer>> FOnlineOfferDescriptionMap;

	/** Mapping of all queried offers to their product information */
	FOnlineOfferDescriptionMap CachedOffers;

	/** The delegate to call for current query */
	FOnQueryOnlineStoreOffersComplete QueryOnlineStoreOffersCompleteDelegate;

	/** Is a query already in flight */
	bool bIsQueryInFlight;
	
	/** Reference to the parent subsystem */
	FOnlineSubsystemGooglePlay* Subsystem;

	/** Delegate fired when a query for purchases has completed, whether successful or unsuccessful */
	FOnQueryForAvailablePurchasesComplete OnQueryForAvailablePurchasesCompleteDelegate;
};

typedef TSharedPtr<FOnlineStoreGooglePlayV2, ESPMode::ThreadSafe> FOnlineStoreGooglePlayV2Ptr;
