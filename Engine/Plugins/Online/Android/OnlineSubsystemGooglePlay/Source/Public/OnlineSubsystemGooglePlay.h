// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GooglePlayGamesWrapper.h"
#include "OnlineSubsystem.h"
#include "OnlineSubsystemImpl.h"
#include "OnlineExternalUIInterfaceGooglePlay.h"
#include "OnlineAsyncTaskManagerGooglePlay.h"
#include "Templates/UniquePtr.h"
#include "HAL/RunnableThread.h"
#include "OnlineStoreGooglePlay.h"
#include "OnlinePurchaseGooglePlay.h"

/** Forward declarations of all interface classes */
typedef TSharedPtr<class FOnlineIdentityGooglePlay,  ESPMode::ThreadSafe> FOnlineIdentityGooglePlayPtr;
typedef TSharedPtr<class FOnlineStoreGooglePlayV2, ESPMode::ThreadSafe> FOnlineStoreGooglePlayV2Ptr;
typedef TSharedPtr<class FOnlinePurchaseGooglePlay, ESPMode::ThreadSafe> FOnlinePurchaseGooglePlayPtr;
typedef TSharedPtr<class FOnlineLeaderboardsGooglePlay, ESPMode::ThreadSafe> FOnlineLeaderboardsGooglePlayPtr;
typedef TSharedPtr<class FOnlineAchievementsGooglePlay, ESPMode::ThreadSafe> FOnlineAchievementsGooglePlayPtr;
typedef TSharedPtr<class FOnlineExternalUIGooglePlay, ESPMode::ThreadSafe> FOnlineExternalUIGooglePlayPtr;

class FRunnableThread;

/**
 * OnlineSubsystemGooglePlay - Implementation of the online subsystem for Google Play services
 */
class ONLINESUBSYSTEMGOOGLEPLAY_API FOnlineSubsystemGooglePlay : 
	public FOnlineSubsystemImpl
{
public:

	virtual ~FOnlineSubsystemGooglePlay() = default;

	//~ Begin IOnlineSubsystem Interface
	virtual IOnlineSessionPtr GetSessionInterface() const override;
	virtual IOnlineFriendsPtr GetFriendsInterface() const override;
	virtual IOnlinePartyPtr GetPartyInterface() const override;
	virtual IOnlineGroupsPtr GetGroupsInterface() const override;
	virtual IOnlineSharedCloudPtr GetSharedCloudInterface() const override;
	virtual IOnlineUserCloudPtr GetUserCloudInterface() const override;
	virtual IOnlineLeaderboardsPtr GetLeaderboardsInterface() const override;
	virtual IOnlineVoicePtr GetVoiceInterface() const  override;
	virtual IOnlineExternalUIPtr GetExternalUIInterface() const override;
	virtual IOnlineTimePtr GetTimeInterface() const override;
	virtual IOnlineIdentityPtr GetIdentityInterface() const override;
	virtual IOnlineTitleFilePtr GetTitleFileInterface() const override;
	virtual IOnlineEntitlementsPtr GetEntitlementsInterface() const override;
	virtual IOnlineStoreV2Ptr GetStoreV2Interface() const override;
	virtual IOnlinePurchasePtr GetPurchaseInterface() const override;
	virtual IOnlineEventsPtr GetEventsInterface() const override { return nullptr; }
	virtual IOnlineMessagePtr GetMessageInterface() const override { return nullptr; }
	virtual IOnlineSharingPtr GetSharingInterface() const override { return nullptr; }
	virtual IOnlineUserPtr GetUserInterface() const override { return nullptr; }
	virtual IOnlineAchievementsPtr GetAchievementsInterface() const override;
	virtual IOnlinePresencePtr GetPresenceInterface() const override { return nullptr; }
	virtual IOnlineChatPtr GetChatInterface() const override { return nullptr; }
	virtual IOnlineStatsPtr GetStatsInterface() const override { return nullptr; }
	virtual IOnlineTurnBasedPtr GetTurnBasedInterface() const override { return nullptr; }
	virtual IOnlineTournamentPtr GetTournamentInterface() const override { return nullptr; }

	virtual class UObject* GetNamedInterface(FName InterfaceName) override { return nullptr; }
	virtual void SetNamedInterface(FName InterfaceName, class UObject* NewInterface) override {}
	virtual bool IsDedicated() const override { return false; }
	virtual bool IsServer() const override { return true; }
	virtual void SetForceDedicated(bool bForce) override {}
	virtual bool IsLocalPlayer(const FUniqueNetId& UniqueId) const override { return true; }

	virtual bool Init() override;
	virtual bool Shutdown() override;
	virtual FString GetAppId() const override;
	virtual bool Exec(class UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;
	virtual bool IsEnabled() const override;
	virtual FText GetOnlineServiceName() const override;
	//~ End IOnlineSubsystem Interface

	virtual bool Tick(float DeltaTime) override;
	
PACKAGE_SCOPE:

	FOnlineSubsystemGooglePlay() = delete;
	explicit FOnlineSubsystemGooglePlay(FName InInstanceName);

	/** Return the async task manager owned by this subsystem */
	class FOnlineAsyncTaskManagerGooglePlay* GetAsyncTaskManager() { return OnlineAsyncTaskThreadRunnable.Get(); }

	/**
	 * Add an async task onto the task queue for processing
	 * @param AsyncTask - new heap allocated task to process on the async task thread
	 */
	void QueueAsyncTask(class FOnlineAsyncTask* AsyncTask);

	FGooglePlayGamesWrapper& GetGooglePlayGamesWrapper() { return GooglePlayGamesWrapper; }

	/** Returns the Google Play-specific version of Identity, useful to avoid unnecessary casting */
	FOnlineIdentityGooglePlayPtr GetIdentityGooglePlay() const { return IdentityInterface; }

	/** Returns the Google Play-specific version of Leaderboards, useful to avoid unnecessary casting */
	FOnlineLeaderboardsGooglePlayPtr GetLeaderboardsGooglePlay() const { return LeaderboardsInterface; }

	/** Returns the Google Play-specific version of Achievements, useful to avoid unnecessary casting */
	FOnlineAchievementsGooglePlayPtr GetAchievementsGooglePlay() const { return AchievementsInterface; }

	/**
	 * Is IAP available for use
	 * @return true if IAP should be available, false otherwise
	 */
	bool IsInAppPurchasingEnabled();

private:

	/** Online async task runnable */
	TUniquePtr<class FOnlineAsyncTaskManagerGooglePlay> OnlineAsyncTaskThreadRunnable;

	/** Online async task thread */
	TUniquePtr<FRunnableThread> OnlineAsyncTaskThread;

	FGooglePlayGamesWrapper GooglePlayGamesWrapper;

	/** Interface to the online identity system */
	FOnlineIdentityGooglePlayPtr IdentityInterface;

	/** Interface to the online catalog */
	FOnlineStoreGooglePlayV2Ptr StoreV2Interface;

	/** Interface to the store purchasing */
	FOnlinePurchaseGooglePlayPtr PurchaseInterface;

	/** Interface to the online leaderboards */
	FOnlineLeaderboardsGooglePlayPtr LeaderboardsInterface;

	/** Interface to the online achievements */
	FOnlineAchievementsGooglePlayPtr AchievementsInterface;

	/** Interface to the external UI services */
	FOnlineExternalUIGooglePlayPtr ExternalUIInterface;
};

typedef TSharedPtr<FOnlineSubsystemGooglePlay, ESPMode::ThreadSafe> FOnlineSubsystemGooglePlayPtr;
