// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

/**
 * Online subsystem module class (Google Play Implementation)
 * Code related to the loading and handling of the Amdroid Google Play module.
 */
class FOnlineSubsystemGooglePlayModule : public IModuleInterface
{
private:
	
	/** Class responsible for creating instance(s) of the subsystem */
	class FOnlineFactoryGooglePlay* GooglePlayFactory;

	virtual ~FOnlineSubsystemGooglePlayModule(){}

	//~ Begin IModuleInterface Interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	virtual bool SupportsDynamicReloading() override
	{
		return false;
	}

	virtual bool SupportsAutomaticShutdown() override
	{
		return false;
	}
	//~ End IModuleInterface Interface
};

typedef TSharedPtr<FOnlineSubsystemGooglePlayModule> FOnlineSubsystemGooglePlayModulePtr;

